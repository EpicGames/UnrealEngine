// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DatasmithExporter : ModuleRules
	{
		public DatasmithExporter(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"DatasmithCore",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"CoreUObject",
					"DirectLink",
					"FreeImage",
					"MeshDescription",
					"MessagingCommon",
					"Projects",
					"RawMesh",
					"StaticMeshDescription",
					"Slate",
					"InputCore",
					"SlateCore",
					"StandaloneRenderer",
				}
			);

			PrivateIncludePathModuleNames.Add("Launch");

			// PrecompileForTargets = PrecompileTargetsType.Any;
		}
	}
}