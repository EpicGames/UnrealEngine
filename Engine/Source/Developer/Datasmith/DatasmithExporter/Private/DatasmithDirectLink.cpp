// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDirectLink.h"

#include "DatasmithExporterManager.h"
#include "IDatasmithSceneElements.h"
#include "DirectLink/DatasmithDirectLinkTools.h"

#include "DirectLinkEndpoint.h"
#include "DirectLinkMisc.h"

#include "Containers/Ticker.h"
#include "MeshDescription.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


DEFINE_LOG_CATEGORY(LogDatasmithDirectLinkExporterAPI);

class FDatasmithDirectLinkImpl
{
public:
	FDatasmithDirectLinkImpl()
		: Endpoint(MakeShared<DirectLink::FEndpoint, ESPMode::ThreadSafe>(TEXT("DatasmithExporter")))
	{
		check(DirectLink::ValidateCommunicationStatus() == DirectLink::ECommunicationStatus::NoIssue);
		Endpoint->SetVerbose();
		// #ue_directlink_integration app specific endpoint name, and source name.
	}

	bool InitializeForScene(const TSharedRef<IDatasmithScene>& Scene)
	{
		UE_LOGF(LogDatasmithDirectLinkExporterAPI, Log, "InitializeForScene");

		Endpoint->RemoveSource(Source);

		// Use the scene's label to name the source
		const TCHAR* SourceName = FCString::Strlen(Scene->GetLabel()) == 0 ? TEXT("unnamed") : Scene->GetLabel();

		Source = Endpoint->AddSource(SourceName, DirectLink::EVisibility::Public);
		bool bSnapshotNow = false;
		Endpoint->SetSourceRoot(Source, &*Scene, bSnapshotNow);
		CurrentScene = Scene;

		return true;
	}

	bool UpdateScene(const TSharedRef<IDatasmithScene>& Scene)
	{
		UE_LOGF(LogDatasmithDirectLinkExporterAPI, Log, "UpdateScene");
		if (CurrentScene != Scene)
		{
			InitializeForScene(Scene);
		}

		Endpoint->SnapshotSource(Source);
		DumpDatasmithScene(Scene, TEXT("send"));
		return true;
	}

	void CloseCurrentSource()
	{
		Endpoint->RemoveSource(Source);
		CurrentScene.Reset();
	}

	TSharedRef<DirectLink::FEndpoint, ESPMode::ThreadSafe> GetEnpoint() const
	{
		return Endpoint;
	}

private:
	TSharedRef<DirectLink::FEndpoint, ESPMode::ThreadSafe> Endpoint;
	DirectLink::FSourceHandle Source;
	TSharedPtr<IDatasmithScene> CurrentScene;
};


TUniquePtr<FDatasmithDirectLinkImpl> DirectLinkImpl;

int32 FDatasmithDirectLink::ValidateCommunicationSetup()
{ return (int32)DirectLink::ValidateCommunicationStatus(); }

bool FDatasmithDirectLink::Shutdown()
{
	DirectLinkImpl.Reset();
	return true;
}

FDatasmithDirectLink::FDatasmithDirectLink()
{
	if (!DirectLinkImpl)
	{
		DirectLinkImpl = MakeUnique<FDatasmithDirectLinkImpl>();
	}
}


bool FDatasmithDirectLink::InitializeForScene(const TSharedRef<IDatasmithScene>& Scene)
{ return DirectLinkImpl->InitializeForScene(Scene); }

bool FDatasmithDirectLink::UpdateScene(const TSharedRef<IDatasmithScene>& Scene)
{ return DirectLinkImpl->UpdateScene(Scene); }

void FDatasmithDirectLink::CloseCurrentSource()
{
	DirectLinkImpl->CloseCurrentSource();
}

TSharedRef<DirectLink::FEndpoint, ESPMode::ThreadSafe> FDatasmithDirectLink::GetEnpoint()
{
	if (!DirectLinkImpl)
	{
		DirectLinkImpl = MakeUnique<FDatasmithDirectLinkImpl>();
	}

	return DirectLinkImpl->GetEnpoint();
}
