// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithExportOptions.h"

EDSExportLightmapUV FDatasmithExportOptions::LightmapUV = EDSExportLightmapUV::Always;

EDSResizeTextureMode FDatasmithExportOptions::ResizeTexturesMode = EDSResizeTextureMode::NearestPowerOfTwo;
EDSResizedTexturesPath FDatasmithExportOptions::PathTexturesMode = EDSResizedTexturesPath::ExportFolder;
int32 FDatasmithExportOptions::MaxTextureSize = 4096;

// In UE, the maximum texture resolution is computed as:
// const int32 MaximumSupportedResolution = 1 << (GMaxTextureMipCount - 1);
// (ref. UTextureFactory::IsImportResolutionValid in EditorFactories.cpp)
// but the value is defined here as to not add a dependency to the RHI module.
const int32 FDatasmithExportOptions::MaxUnrealSupportedTextureSize = 8192;

float FDatasmithExportOptions::ColorGamma = 2.2f;
