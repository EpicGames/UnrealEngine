// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithExporterManager.h"

#include "Modules/ModuleManager.h"
#include "UObject/GarbageCollection.h"

#if IS_PROGRAM
#include "DirectLinkModule.h"

#include "Async/Future.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Queue.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/Event.h"
#include "HAL/RunnableThread.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AssertionMacros.h"
#include "RequiredProgramMainCPPInclude.h"
#include "StandaloneRenderer.h"
#include "Stats/StatsSystem.h"

#ifndef UE_BUILD_DEVELOPMENT_WITH_DEBUGGAME
	#define UE_BUILD_DEVELOPMENT_WITH_DEBUGGAME 0
#endif

// Not using IMPLEMENT_APPLICATION here because it overrides new and delete which can cause issues in some host softwares.
TCHAR GInternalProjectName[64] = TEXT( "UnrealDatasmithExporter" );
// Create the engine at a directory that is null before the initialization of datasmith is done
const TCHAR *GForeignEngineDir = nullptr;
FEngineLoop GEngineLoop;
#endif // IS_PROGRAM



IMPLEMENT_MODULE(FDefaultModuleImpl, DatasmithExporter)

#if IS_PROGRAM
/**
 * A simple runnable to host and init a simple engine loop
 */
class FDatasmithGameThread : public FRunnable
{
public:
	FDatasmithGameThread(FString&& InPreInitCommandArgs, const FDatasmithExporterManager::FInitOptions& InForeignEngineDir);
	virtual uint32 Run() override;
	TFuture<bool> GetOnInitDoneFuture();

	void RequestExit();

	bool WasInitializedWithMessaging() const { return bUseMessaging; }

	TQueue<FSimpleDelegate, EQueueMode::Mpsc> CommandQueue;

	FEvent* SyncEvent;

private:
	void OnInit();
	void Shutdown();

	bool bKeepRunning;
	bool bUseMessaging;
	bool bSuppressLogs;
	TUniquePtr<TPromise<bool>> InitDone;
	FString PreInitCommandArgs;
	FString ForeignEngineDir;
};

bool FDatasmithExporterManager::bEngineInitialized = false;
bool FDatasmithExporterManager::bUseMessaging = false;
FRunnableThread* FDatasmithExporterManager::GMainThreadAsRunnable = nullptr;
TSharedPtr<FDatasmithGameThread> FDatasmithExporterManager::GDatasmithGameThread;

class FDatasmithExportOutputDevice : public FOutputDevice
{
public:
	FDatasmithExportOutputDevice()
	{
	}

	virtual ~FDatasmithExportOutputDevice()
	{
	}

	virtual void Serialize(const TCHAR* /*V*/, ELogVerbosity::Type /*Verbosity*/, const class FName& /*Category*/) override
	{
	}
};


namespace DatasmithGameThread
{
	bool InitializeInCurrentThread(const FString& PreInitCommandArgs, bool bSuppressLogs)
	{
		bool bEngineInitialized = GEngineLoop.PreInit(*PreInitCommandArgs) == 0;

		// Workaround for UdpMessaging not starting processing until PostDefault stage - UE-179092
		// And since Datasmith plugins are built without WITH_ENGINE FEngineLoop::LoadStartupModules is not called which effectively skips PostDefault stage
		IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostDefault);

		// Make sure all UObject classes are registered and default properties have been initialized
		ProcessNewlyLoadedUObjects();

		// Tell the module manager it may now process newly-loaded UObjects when new C++ modules are loaded
		FModuleManager::Get().StartProcessingNewlyLoadedObjects();

		if ( GLog )
		{
			// Make sure Logger is set on the right thread
			GLog->SetCurrentThreadAsPrimaryThread();

			if ( bSuppressLogs )
			{
				// Clean up existing output devices
				GLog->TearDown();

				// Add Datasmith output device
				GLog->AddOutputDevice(new FDatasmithExportOutputDevice());
			}
		}

		return bEngineInitialized;
	}

	void ShutdownInCurrentThread()
	{
		// Some adjustments must be made if shutdown thread differ from the one Initialize has been called from
		if (GGameThreadId != FPlatformTLS::GetCurrentThreadId())
		{
			// Force Game thread to be the current one
			GGameThreadId = FPlatformTLS::GetCurrentThreadId();
			// Set the current thread on the Logger
			if ( GLog )
			{
				GLog->SetCurrentThreadAsPrimaryThread();
			}

			// Re-attach task graph to new game thread
			FTaskGraphInterface::Get().AttachToThread(ENamedThreads::GameThread);
		}

		// Close the logging device
		if ( GLog )
		{
			GLog->TearDown();
		}

		// Proceed with process exit
		GEngineLoop.AppPreExit();
		GEngineLoop.AppExit();
	}
}

FDatasmithGameThread::FDatasmithGameThread(FString&& InPreInitCommandArgs, const FDatasmithExporterManager::FInitOptions& InitOptions)
	: FRunnable()
	, bKeepRunning(true)
	, bUseMessaging(InitOptions.bEnableMessaging)
	, bSuppressLogs(InitOptions.bSuppressLogs)
	, InitDone(MakeUnique<TPromise<bool>>())
	, PreInitCommandArgs(MoveTemp(InPreInitCommandArgs))
	, ForeignEngineDir(InitOptions.RemoteEngineDirPath)
{
}

uint32 FDatasmithGameThread::Run()
{
	// We need to explicitly flag this thread as the GameThread to avoid errors.
	FTaskTagScope Scope(ETaskTag::EGameThread);

	// init
	bKeepRunning = true;
	OnInit();


	while (bKeepRunning)
	{
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		UE::Stats::FStats::AdvanceFrame(false);
		FTSTicker::GetCoreTicker().Tick(FApp::GetDeltaTime());
		FSlateApplication::Get().PumpMessages();
		FSlateApplication::Get().Tick();

		double LastSlateUpdateTime = FPlatformTime::Seconds();

		FSimpleDelegate Command;
		while (CommandQueue.Dequeue( Command ) && bKeepRunning)
		{
			Command.ExecuteIfBound();
		}

		// Update this if we can get refresh rate of the user monitor(s)
		constexpr double FrameTime = 1.0 / 60;
		double TimeSinceLastUpdate =  FPlatformTime::Seconds() - LastSlateUpdateTime;
		while (TimeSinceLastUpdate < FrameTime && bKeepRunning)
		{
			while (CommandQueue.Dequeue( Command ) && bKeepRunning)
			{
				Command.ExecuteIfBound();
			}

			if (bKeepRunning)
			{
				double CurrentTime = FPlatformTime::Seconds();
				SyncEvent->Wait( FTimespan::FromSeconds(FrameTime - TimeSinceLastUpdate) );
				TimeSinceLastUpdate += FPlatformTime::Seconds() - CurrentTime;
			}
		}
	}

	// exit
	Shutdown();
	return 0;
}

TFuture<bool> FDatasmithGameThread::GetOnInitDoneFuture()
{
	if ( InitDone )
	{
		return InitDone->GetFuture();
	}

	// If you got there the engine loop was already initialized
	checkNoEntry();
	return {};
}

void FDatasmithGameThread::RequestExit()
{
	FSimpleDelegate RunOnGameThread;
	RunOnGameThread.BindLambda([this]()
		{
			bKeepRunning = false;
		});
	FDatasmithExporterManager::PushCommandIntoGameThread(MoveTemp(RunOnGameThread), true);
}

void FDatasmithGameThread::OnInit()
{
	// Make sure that external engine path is normalized and ends with a '/'
	FPaths::NormalizeDirectoryName(ForeignEngineDir);
	if (!ForeignEngineDir.IsEmpty() && ForeignEngineDir[ForeignEngineDir.Len() - 1] != TEXT('/'))
	{
		ForeignEngineDir += TEXT("/");
	}

	GForeignEngineDir = *ForeignEngineDir;
	bool bInitSucceded = DatasmithGameThread::InitializeInCurrentThread(PreInitCommandArgs, bSuppressLogs);

	// The macro UE_EXTERNAL_PROFILING_ENABLED must be define to 0 otherwise the engine path will be already initialized with the wrong path
	check(ForeignEngineDir == FGenericPlatformMisc::EngineDir());

	PreInitCommandArgs.Empty();
	if (bInitSucceded)
	{
		SyncEvent = FPlatformProcess::GetSynchEventFromPool();

		PreInitCommandArgs.Empty();

		// Initialize a normal Slate application using the platform's standalone renderer
		FSlateApplication::InitializeAsStandaloneApplication( GetStandardStandaloneRenderer() );

		// Load the required module at boot (User calls can't load modules on their thread
		FModuleManager& ModuleManager = FModuleManager::Get();

		ModuleManager.LoadModuleChecked(FName(TEXT("DatasmithExporterUI")));

		if (bUseMessaging)
		{
			// Init DirectLink module (and dependencies)
			FDirectLinkModule::Get();
		}
	}
	else
	{
		checkf(false, TEXT("Couldn't pre-init engine in the custom datasmith game thread."))
	}

	InitDone->SetValue(bInitSucceded);

	/**
	 * Free the promise as soon a possible. It can cause some issue when done after the shutdown.
	 * The destructor require a lazy initialized global event pool that don't survive the shutdown.
	 */
	InitDone.Reset();
}

void FDatasmithGameThread::Shutdown()
{
	FSlateApplication::Get().CloseAllWindowsImmediately();
	FSlateApplication::Shutdown();

	FPlatformProcess::ReturnSynchEventToPool( SyncEvent );

	DatasmithGameThread::ShutdownInCurrentThread();
}
#endif // IS_PROGRAM


bool FDatasmithExporterManager::Initialize()
{
	return Initialize(FInitOptions());
}

bool FDatasmithExporterManager::Initialize(const FInitOptions& InitOptions)
{
#if IS_PROGRAM
	if ( bEngineInitialized )
	{
		return false;
	}

	if (!bEngineInitialized)
	{
		FString CmdLine;

		bUseMessaging = InitOptions.bEnableMessaging;

		if (InitOptions.bEnableMessaging)
		{
			CmdLine += TEXT(" -Messaging");
		}

		if (InitOptions.bSaveLogToUserDir)
		{
			CmdLine += TEXT(" -SaveToUserDir");
		}

		if (InitOptions.bSuppressLogs)
		{
			CmdLine += TEXT(" -logcmds=\"Global None\"");
		}

		if (InitOptions.bUseDatasmithExporterUI)
		{
			checkf(InitOptions.RemoteEngineDirPath != nullptr, TEXT("Datasmith exporter UI need a path to its minimal engine folder"));

			// Start a custom game thread
			GDatasmithGameThread = MakeShared<FDatasmithGameThread>(MoveTemp(CmdLine), InitOptions);
			TFuture<bool> OnInitDoneFuture = GDatasmithGameThread->GetOnInitDoneFuture();
			GMainThreadAsRunnable = FRunnableThread::Create(GDatasmithGameThread.Get(), TEXT("DatasmithMainThread"));
			bEngineInitialized = OnInitDoneFuture.Get();

			// Remove the ETaskTag::EStaticInit tag on the current thread as it can cause this thread to compete with the main thread we just created.
			FTaskTagScope::SetTagNone();
		}
		else
		{
			FTaskTagScope Scope(ETaskTag::EGameThread);
			bEngineInitialized = DatasmithGameThread::InitializeInCurrentThread(CmdLine, InitOptions.bSuppressLogs);
		}

		return bEngineInitialized;
	}
#endif //IS_PROGRAM
	return true;
}

void FDatasmithExporterManager::Shutdown()
{
#if IS_PROGRAM
	if (bEngineInitialized)
	{
		if (GDatasmithGameThread)
		{
			GDatasmithGameThread->RequestExit();
			GMainThreadAsRunnable->WaitForCompletion();
			delete GMainThreadAsRunnable;
			GDatasmithGameThread.Reset();
		}
		else
		{
			DatasmithGameThread::ShutdownInCurrentThread();
		}

		bEngineInitialized = false;
	}
#endif // IS_PROGRAM
}

bool FDatasmithExporterManager::RunGarbageCollection()
{
#if IS_PROGRAM
	if (GDatasmithGameThread)
	{
		FSimpleDelegate Command;
		TSharedRef<TPromise<void>> SharedPromise = MakeShared<TPromise<void>>();
		TFuture<void> GarbageCollectionDoneFuture = SharedPromise->GetFuture();
		Command.BindLambda([SharedPromise]()
			{
				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
				SharedPromise->SetValue();
			});
		PushCommandIntoGameThread(MoveTemp(Command), true);

		GarbageCollectionDoneFuture.Wait();
		return true;
	}
	else if (IsInGameThread())
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		return true;
	}

	return false;
#else
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	return true;
#endif
}

#if IS_PROGRAM
void FDatasmithExporterManager::PushCommandIntoGameThread(FSimpleDelegate&& Command, bool bWakeUpGameThread)
{
	if (GDatasmithGameThread)
	{
		GDatasmithGameThread->CommandQueue.Enqueue(MoveTemp(Command));
		if ( bWakeUpGameThread && GDatasmithGameThread->SyncEvent )
		{
			GDatasmithGameThread->SyncEvent->Trigger();
		}
	}
	else if (IsInGameThread())
	{
		Command.ExecuteIfBound();
	}
	else
	{
		// The exporter manager wasn't properly initialized.
		checkNoEntry();
	}
}

bool FDatasmithExporterManager::WasInitializedWithMessaging()
{
	if( GDatasmithGameThread )
	{
		return GDatasmithGameThread->WasInitializedWithMessaging();
	}

	return bUseMessaging && bEngineInitialized;
}

bool FDatasmithExporterManager::WasInitializedWithGameThread()
{
	return GDatasmithGameThread && bEngineInitialized;
}
#endif

