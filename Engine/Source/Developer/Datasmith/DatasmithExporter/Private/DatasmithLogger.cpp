// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithLogger.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"

class FDatasmithLoggerImpl
{
public:
	TArray<FString> GeneralErrors;
	TArray<FString> TextureErrors;
	TArray<FString> MissingAssetErrors;
	FString ErrorMessage;
};

FDatasmithLogger::FDatasmithLogger()
	: Impl( new FDatasmithLoggerImpl() )
{
}

FDatasmithLogger::~FDatasmithLogger()
{
	delete Impl;
}

void FDatasmithLogger::AddGeneralError(const TCHAR* InError)
{
	for (int32 i = 0; i < Impl->GeneralErrors.Num(); i++)
	{
		if (InError == Impl->GeneralErrors[i])
		{
			return;
		}
	}

	Impl->GeneralErrors.Add(InError);
}

int32 FDatasmithLogger::GetGeneralErrorsCount()
{
	return Impl->GeneralErrors.Num();
}

const TCHAR* FDatasmithLogger::GetGeneralError( int32 Index ) const
{
	return  *Impl->GeneralErrors[ Index ];
}

void FDatasmithLogger::ResetGeneralErrors()
{
	Impl->GeneralErrors.Empty();
}

void FDatasmithLogger::AddTextureError(const TCHAR* InError)
{
	for (int32 i = 0; i < Impl->TextureErrors.Num(); i++)
	{
		if (InError == Impl->TextureErrors[i])
		{
			return;
		}
	}

	Impl->TextureErrors.Add(InError);
}

int32 FDatasmithLogger::GetTextureErrorsCount()
{
	return Impl->TextureErrors.Num();
}

const TCHAR* FDatasmithLogger::GetTextureError(int32 Index) const
{
	return  *Impl->TextureErrors[Index];
}

void FDatasmithLogger::ResetTextureErrors()
{
	Impl->TextureErrors.Empty();
}

void FDatasmithLogger::AddMissingAssetError(const TCHAR* InError)
{	
	Impl->MissingAssetErrors.AddUnique(InError);
}

int32 FDatasmithLogger::GetMissingAssetErrorsCount()
{
	return Impl->MissingAssetErrors.Num();
}

const TCHAR* FDatasmithLogger::GetMissingAssetError(int32 Index) const
{
	return  *Impl->MissingAssetErrors[Index];
}

void FDatasmithLogger::ResetMissingAssetErrors()
{
	Impl->MissingAssetErrors.Empty();
}

