// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMeshExporter.h"

#include "DatasmithCore.h"
#include "DatasmithCloth.h"  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
#include "DatasmithExporterManager.h"
#include "DatasmithMesh.h"
#include "DatasmithMeshUObject.h"
#include "DatasmithMeshSerialization.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"

#include "Containers/LockFreeList.h"
#include "HAL/FileManager.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"


struct FDatasmithMeshExporterOptions
{
	FDatasmithMeshExporterOptions(const FString& InFullPath, FDatasmithMesh& InMesh, EDSExportLightmapUV InLightmapUV, FDatasmithMesh* InCollisionMesh = nullptr)
		: MeshFullPath(InFullPath)
		, Mesh(InMesh)
		, LightmapUV(InLightmapUV)
		, CollisionMesh(InCollisionMesh)
	{}

	FString MeshFullPath;
	FDatasmithMesh& Mesh;
	EDSExportLightmapUV LightmapUV;
	FDatasmithMesh* CollisionMesh;
};


class FDatasmithMeshExporterImpl
{
public:
	bool DoExport(TSharedPtr<IDatasmithMeshElement>& MeshElement, const FDatasmithMeshExporterOptions& ExportOptions);
	FString LastError;

private:
	bool WriteMeshFile(const FDatasmithMeshExporterOptions& ExporterOptions, FMD5Hash& OutHash);
};


bool FDatasmithMeshExporterImpl::DoExport(TSharedPtr<IDatasmithMeshElement>& MeshElement, const FDatasmithMeshExporterOptions& ExportOptions)
{
	FDatasmithMesh& Mesh = ExportOptions.Mesh;

	// If the mesh doesn't have a name, use the filename as its name
	if (FCString::Strlen(Mesh.GetName()) == 0)
	{
		Mesh.SetName(*FPaths::GetBaseFilename(ExportOptions.MeshFullPath));
	}
	FMD5Hash Hash;
	if (WriteMeshFile(ExportOptions, Hash))
	{
		// If no existing MeshElement provided, create one.
		if (!MeshElement)
		{
			MeshElement = FDatasmithSceneFactory::CreateMesh(Mesh.GetName());
		}
		MeshElement->SetFile(*ExportOptions.MeshFullPath);
		MeshElement->SetFileHash(Hash);

		FBox3f Extents = Mesh.GetExtents();
		float Width = Extents.Max[0] - Extents.Min[0];
		float Height = Extents.Max[2] - Extents.Min[2];
		float Depth = Extents.Max[1] - Extents.Min[1];

		MeshElement->SetDimensions(Mesh.ComputeArea(), Width, Height, Depth);
		MeshElement->SetLightmapSourceUV(Mesh.GetLightmapSourceUVChannel());

		return true;
	}
	else
	{
		UE_LOGF(LogDatasmith, Warning, "Cannot export mesh %ls: %ls", Mesh.GetName(), *LastError);
	}

	return false;
}



FDatasmithMeshExporter::FDatasmithMeshExporter()
	: Impl(MakeUnique<FDatasmithMeshExporterImpl>())
{}


FDatasmithMeshExporter::~FDatasmithMeshExporter() = default;


FString GetMeshFilePath(const TCHAR* Filepath, const TCHAR* Filename)
{
	FString NormalizedFilepath = Filepath;
	FPaths::NormalizeDirectoryName(NormalizedFilepath);

	FString NormalizedFilename = Filename;
	FPaths::NormalizeFilename(NormalizedFilename);

	return FPaths::Combine(*NormalizedFilepath, FPaths::SetExtension(NormalizedFilename, UDatasmithMesh::GetFileExtension()));
}


bool FDatasmithMeshExporterImpl::WriteMeshFile(const FDatasmithMeshExporterOptions& ExporterOptions, FMD5Hash& OutHash)
{
	FDatasmithPackedMeshes Pack;

	auto PackMeshModels = [&](FDatasmithMesh& Mesh, bool bIsCollisionMesh) -> FDatasmithMeshModels
	{
		FDatasmithMeshModels Models;
		Models.bIsCollisionMesh = bIsCollisionMesh;
		Models.MeshName = Mesh.GetName();

		Models.SourceModels.Reserve(Mesh.GetLODsCount() + 1);
		FMeshDescription& BaseMeshDescription = Models.SourceModels.AddDefaulted_GetRef();
		FDatasmithMeshUtils::ToMeshDescription(Mesh, BaseMeshDescription, FDatasmithMeshUtils::GenerateBox);

		for (int32 LodIndex = 0; LodIndex < Mesh.GetLODsCount(); ++LodIndex)
		{
			if (FDatasmithMesh* LodMesh = Mesh.GetLOD(LodIndex))
			{
				FMeshDescription& LodMeshDescription = Models.SourceModels.AddDefaulted_GetRef();
				FDatasmithMeshUtils::ToMeshDescription(*LodMesh, LodMeshDescription, FDatasmithMeshUtils::GenerateBox);
			}
		}

		return Models;
	};

	Pack.Meshes.Add(PackMeshModels(ExporterOptions.Mesh, false));
	if (ExporterOptions.CollisionMesh)
	{
		Pack.Meshes.Add(PackMeshModels(*ExporterOptions.CollisionMesh, true));
	}

	TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileWriter(*ExporterOptions.MeshFullPath));
	if (!Archive.IsValid())
	{
		LastError = FString::Printf(TEXT("Failed writing to file %s"), *ExporterOptions.MeshFullPath);
		return false;
	}

	int32 LegacyMeshCount = 0;
	*Archive << LegacyMeshCount; // the legacy importer expect a mesh count on the first bytes. Just in case a new file would end up parsed by the legacy code...
	OutHash = Pack.Serialize(*Archive);

	return !Archive->IsError();
}


TSharedPtr<IDatasmithMeshElement> FDatasmithMeshExporter::ExportToUObject(const TCHAR* Filepath, const TCHAR* Filename, FDatasmithMesh& Mesh, FDatasmithMesh* CollisionMesh, EDSExportLightmapUV LightmapUV)
{
	FString FullPath(GetMeshFilePath(Filepath, Filename));

	TSharedPtr<IDatasmithMeshElement> ExportedMeshElement;
	FDatasmithMeshExporterOptions ExportOptions(FullPath, Mesh, LightmapUV, CollisionMesh);
	Impl->DoExport(ExportedMeshElement, ExportOptions);

	return ExportedMeshElement;
}


bool FDatasmithMeshExporter::ExportToUObject(TSharedPtr<IDatasmithMeshElement>& MeshElement, const TCHAR* Filepath, FDatasmithMesh& Mesh, FDatasmithMesh* CollisionMesh, EDSExportLightmapUV LightmapUV)
{
	FString FullPath(GetMeshFilePath(Filepath, MeshElement->GetName()));
	FDatasmithMeshExporterOptions ExportOptions(FullPath, Mesh, LightmapUV, CollisionMesh);

	return Impl->DoExport(MeshElement, ExportOptions);
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FDatasmithMeshExporter::ExportCloth(FDatasmithCloth& Cloth, TSharedPtr<IDatasmithClothElement>& ClothElement, const TCHAR* FilePath, const TCHAR* AssetsOutputPath) const  // UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
{
	FString Path = FPaths::SetExtension(FilePath, TEXT(".udscloth"));
	FPaths::NormalizeFilename(Path);
	FString RelativeFilePath = Path;
	if (AssetsOutputPath)
	{
		FString Tmp(AssetsOutputPath);
		Tmp += TEXT('/');
		FPaths::MakePathRelativeTo(RelativeFilePath, *Tmp);
	}
	ClothElement->SetFile(*RelativeFilePath);

	TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileWriter(*Path));
	if (!Archive.IsValid())
	{
		Impl->LastError = FString::Printf(TEXT("Failed writing to file %s"), *Path);
		return false;
	}

	FDatasmithPackedCloths Pack;
	FDatasmithClothInfo& Info = Pack.ClothInfos.AddDefaulted_GetRef();
	Info.Cloth = Cloth;
	FMD5Hash OutHash = Pack.Serialize(*Archive);

	return true;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FString FDatasmithMeshExporter::GetLastError() const
{
	return Impl->LastError;
}

