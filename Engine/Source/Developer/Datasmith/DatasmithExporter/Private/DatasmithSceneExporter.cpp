// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSceneExporter.h"

#include "DatasmithAnimationSerializer.h"
#include "DatasmithExportOptions.h"
#include "DatasmithExporterManager.h"
#include "DatasmithLogger.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithProgressManager.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneXmlWriter.h"
#include "DatasmithUtils.h"

#include "GenericPlatform/GenericPlatformFile.h"
#include "Containers/Array.h"
#include "Algo/Find.h"
#include "Containers/Set.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "UObject/GarbageCollection.h"

// Hash function to use FMD5Hash in TMap
inline uint32 GetTypeHash(const FMD5Hash& Hash)
{
	uint32* HashAsInt32 = (uint32*)Hash.GetBytes();
	return HashAsInt32[0] ^ HashAsInt32[1] ^ HashAsInt32[2] ^ HashAsInt32[3];
}

class FDatasmithSceneExporterImpl
{
public:
	FDatasmithSceneExporterImpl();

	// call this before export the actual bitmaps
	void CheckBumpMaps( TSharedRef< IDatasmithScene > DatasmithScene );
	void UpdateTextureElements( TSharedRef< IDatasmithScene > DatasmithScene );
	void UpdateAssetOutputPath();

	static EDatasmithTextureMode GetTextureModeFromPropertyName(const FString& PropertyName);
	static FString GetFileNameWithHash(const FString& FullPath);

	FString Name;
	FString OutputPath;
	FString AssetsOutputPath;

	uint64 ExportStartCycles;

	TSharedPtr<IDatasmithProgressManager> ProgressManager;
	TSharedPtr<FDatasmithLogger> Logger;
};

FDatasmithSceneExporterImpl::FDatasmithSceneExporterImpl()
	: ExportStartCycles(0)
{
}

void FDatasmithSceneExporterImpl::UpdateTextureElements( TSharedRef< IDatasmithScene > DatasmithScene )
{
	FDatasmithTextureUtils::CalculateTextureHashes(DatasmithScene);

	// No need to do anything if user required to keep images at original location or no output path is set
	if (FDatasmithExportOptions::PathTexturesMode == EDSResizedTexturesPath::OriginalFolder || AssetsOutputPath.IsEmpty())
	{
		return;
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	TMap<FMD5Hash, FString> HashFilePathMap;
	FDatasmithUniqueNameProvider TextureFileNameProvider;

	for (int32 i = 0; i < DatasmithScene->GetTexturesCount(); i++)
	{
		TSharedPtr< IDatasmithTextureElement > TextureElement = DatasmithScene->GetTexture(i);

		FString TextureFileName = TextureElement->GetFile();

		float RatioDone = float(i + 1) / float( DatasmithScene->GetTexturesCount() );
		if (ProgressManager.IsValid())
		{
			ProgressManager->ProgressEvent(RatioDone, *FPaths::GetBaseFilename( TextureFileName ));
		}

		FString& NewFilename = HashFilePathMap.FindOrAdd(TextureElement->GetFileHash());

		// If this texture has not been exported yet, find a unique name for it and copy its file to the asset output path.
		if (NewFilename.IsEmpty())
		{
			const FString UniqueFileName = TextureFileNameProvider.GenerateUniqueName( FPaths::GetBaseFilename( TextureFileName ) );
			TextureFileNameProvider.AddExistingName( UniqueFileName );
			const FString FileExtension = FPaths::GetExtension(TextureFileName, /*bIncludeDot=*/true);
			NewFilename = FPaths::Combine(AssetsOutputPath, UniqueFileName + FileExtension);

			// Copy image file to new location
			if (!FPaths::IsSamePath(*NewFilename, *TextureFileName))
			{
				PlatformFile.CopyFile(*NewFilename, *TextureFileName);
			}
		}

		// Update texture element
		if (TextureFileName != NewFilename)
		{
			TextureElement->SetFile(*NewFilename);
		}
	}
}

void FDatasmithSceneExporterImpl::CheckBumpMaps( TSharedRef< IDatasmithScene > DatasmithScene )
{
	for ( int32 MaterialIndex = 0; MaterialIndex < DatasmithScene->GetMaterialsCount(); ++MaterialIndex )
	{
		TSharedPtr< IDatasmithBaseMaterialElement > BaseMaterialElement = DatasmithScene->GetMaterial( MaterialIndex );

		if ( BaseMaterialElement->IsA( EDatasmithElementType::Material ) )
		{
			const TSharedPtr< IDatasmithMaterialElement >& MaterialElement = StaticCastSharedPtr< IDatasmithMaterialElement >( BaseMaterialElement );

			for (int32 j = 0; j < MaterialElement->GetShadersCount(); ++j )
			{
				TSharedPtr< IDatasmithShaderElement >& Shader = MaterialElement->GetShader(j);

				if (Shader->GetBumpComp()->GetMode() == EDatasmithCompMode::Regular && Shader->GetBumpComp()->GetParamSurfacesCount() == 1 &&
					Shader->GetNormalComp()->GetParamSurfacesCount() == 0)
				{
					FString TextureName = Shader->GetBumpComp()->GetParamTexture(0);
					FString NormalTextureName = TextureName + TEXT("_Norm");

					if ( !TextureName.IsEmpty() )
					{
						FDatasmithTextureSampler UVs = Shader->GetBumpComp()->GetParamTextureSampler(0);

						TSharedPtr< IDatasmithTextureElement > TextureElement;
						TSharedPtr< IDatasmithTextureElement > NormalTextureElement;
						for ( int32 TextureIndex = 0; TextureIndex < DatasmithScene->GetTexturesCount(); ++TextureIndex )
						{
							if ( DatasmithScene->GetTexture( TextureIndex )->GetName() == TextureName )
							{
								TextureElement = DatasmithScene->GetTexture( TextureIndex );
							}
							else if ( DatasmithScene->GetTexture( TextureIndex )->GetName() == NormalTextureName )
							{
								NormalTextureElement = DatasmithScene->GetTexture( TextureIndex );
							}
						}

						if ( TextureElement )
						{
							if ( !NormalTextureElement )
							{
								NormalTextureElement = FDatasmithSceneFactory::CreateTexture( *NormalTextureName );

								NormalTextureElement->SetRGBCurve( 1.f );
								NormalTextureElement->SetFile( TextureElement->GetFile() );
								NormalTextureElement->SetFileHash( TextureElement->GetFileHash() );
								NormalTextureElement->SetTextureMode( EDatasmithTextureMode::Bump );

								DatasmithScene->AddTexture( NormalTextureElement );
							}

							Shader->GetNormalComp()->AddSurface( *NormalTextureName, UVs );
							Shader->GetBumpComp()->ClearSurface();
						}
					}
				}
			}
		}
	}
}

void FDatasmithSceneExporterImpl::UpdateAssetOutputPath()
{
	if (Name.IsEmpty())
	{
		// Just set the AssetsOutputPath to OutputPath, if the scene exporter has not been named
		AssetsOutputPath = OutputPath;
	}
	else if (!OutputPath.IsEmpty())
	{
		AssetsOutputPath = FPaths::Combine(OutputPath, Name + TEXT("_Assets"));
	}
}

EDatasmithTextureMode FDatasmithSceneExporterImpl::GetTextureModeFromPropertyName(const FString& PropertyName)
{
	if (PropertyName.Find(TEXT("BUMP")) != INDEX_NONE)
	{
		return EDatasmithTextureMode::Bump;
	}
	else if (PropertyName.Find(TEXT("SPECULAR")) != INDEX_NONE)
	{
		return EDatasmithTextureMode::Specular;
	}
	else if (PropertyName.Find(TEXT("NORMAL")) != INDEX_NONE)
	{
		return EDatasmithTextureMode::Normal;
	}

	return EDatasmithTextureMode::Diffuse;
};

FString FDatasmithSceneExporterImpl::GetFileNameWithHash(const FString& FullPath)
{
	FString Hash = FMD5::HashAnsiString(*FullPath);
	FString FileName = FPaths::GetBaseFilename(FullPath);
	FString Extension = FPaths::GetExtension(FileName);
	FileName = FileName + TEXT("_") + Hash + Extension;

	return FileName;
}

FDatasmithSceneExporter::FDatasmithSceneExporter()
	: Impl( MakeUnique< FDatasmithSceneExporterImpl >() )
{
}

FDatasmithSceneExporter::~FDatasmithSceneExporter() = default;


void FDatasmithSceneExporter::PreExport()
{
	// Collect start time to log amount of time spent to export scene
	Impl->ExportStartCycles = FPlatformTime::Cycles64();
}

void FDatasmithSceneExporter::Export( TSharedRef< IDatasmithScene > DatasmithScene, bool bCleanupUnusedElements )
{
	if ( Impl->ExportStartCycles == 0 )
	{
		Impl->ExportStartCycles = FPlatformTime::Cycles64();
	}

	FString FilePath = FPaths::Combine(Impl->OutputPath, Impl->Name ) + TEXT(".") + FDatasmithUtils::GetFileExtension();

	TUniquePtr<FArchive> Archive( IFileManager::Get().CreateFileWriter( *FilePath ) );

	if ( !Archive.IsValid() )
	{
		if ( Impl->Logger.IsValid() )
		{
			Impl->Logger->AddGeneralError( *( TEXT("Unable to create file ") + FilePath + TEXT(", Aborting the export process") ) );
		}
		return;
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PlatformFile.CreateDirectoryTree( *Impl->AssetsOutputPath );

	// Add Bump maps from Material objects to scene as TextureElement
	Impl->CheckBumpMaps( DatasmithScene );

	FDatasmithSceneUtils::CleanUpScene(DatasmithScene, bCleanupUnusedElements);

	// Update TextureElements
	Impl->UpdateTextureElements( DatasmithScene );

	// Convert paths to relative
	FString AbsoluteDir = Impl->OutputPath + TEXT("/");

	for ( int32 MeshIndex = 0; MeshIndex < DatasmithScene->GetMeshesCount(); ++MeshIndex )
	{
		TSharedPtr< IDatasmithMeshElement > Mesh = DatasmithScene->GetMesh( MeshIndex );

		FString RelativePath = Mesh->GetFile();
		FPaths::MakePathRelativeTo( RelativePath, *AbsoluteDir );

		Mesh->SetFile( *RelativePath );
	}

	for ( int32 TextureIndex = 0; TextureIndex < DatasmithScene->GetTexturesCount(); ++TextureIndex )
	{
		TSharedPtr< IDatasmithTextureElement > Texture = DatasmithScene->GetTexture( TextureIndex );

		FString TextureFile = Texture->GetFile();
		FPaths::MakePathRelativeTo( TextureFile, *AbsoluteDir );
		Texture->SetFile( *TextureFile );
	}

	FDatasmithAnimationSerializer AnimSerializer;
	int32 NumSequences = DatasmithScene->GetLevelSequencesCount();
	for (int32 SequenceIndex = 0; SequenceIndex < NumSequences; ++SequenceIndex)
	{
		const TSharedPtr<IDatasmithLevelSequenceElement>& LevelSequence = DatasmithScene->GetLevelSequence(SequenceIndex);
		if (LevelSequence.IsValid())
		{
			FString AnimFilePath = FPaths::Combine(Impl->AssetsOutputPath, LevelSequence->GetName()) + DATASMITH_ANIMATION_EXTENSION;

			if (AnimSerializer.Serialize(LevelSequence.ToSharedRef(), *AnimFilePath))
			{
				TUniquePtr<FArchive> AnimArchive(IFileManager::Get().CreateFileReader(*AnimFilePath));
				if (AnimArchive)
				{
					LevelSequence->SetFileHash(FMD5Hash::HashFileFromArchive(AnimArchive.Get()));
				}

				FPaths::MakePathRelativeTo(AnimFilePath, *AbsoluteDir);
				LevelSequence->SetFile(*AnimFilePath);
			}
		}
	}

	// Log time spent to export scene in seconds
	int ElapsedTime = (int)FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - Impl->ExportStartCycles);
	DatasmithScene->SetExportDuration( ElapsedTime );

	FDatasmithSceneXmlWriter DatasmithSceneXmlWriter;
	DatasmithSceneXmlWriter.Serialize( DatasmithScene, *Archive );

	Archive->Close();

	// Run the garbage collector at this point so that we're in a good state for the next export
	FDatasmithExporterManager::RunGarbageCollection();
}

void FDatasmithSceneExporter::Reset()
{
	Impl->ProgressManager = nullptr;
	Impl->Logger = nullptr;

	Impl->ExportStartCycles = 0;
}

void FDatasmithSceneExporter::SetProgressManager( const TSharedPtr< IDatasmithProgressManager >& InProgressManager )
{
	Impl->ProgressManager = InProgressManager;
}

void FDatasmithSceneExporter::SetLogger( const TSharedPtr< FDatasmithLogger >& InLogger )
{
	Impl->Logger = InLogger;
}

void FDatasmithSceneExporter::SetName(const TCHAR* InName)
{
	Impl->Name = InName;
	Impl->UpdateAssetOutputPath();
}

const TCHAR* FDatasmithSceneExporter::GetName() const
{
	return *Impl->Name;
}

void FDatasmithSceneExporter::SetOutputPath( const TCHAR* InOutputPath )
{
	Impl->OutputPath = InOutputPath;
	FPaths::NormalizeDirectoryName( Impl->OutputPath );
	Impl->UpdateAssetOutputPath();
}

const TCHAR* FDatasmithSceneExporter::GetOutputPath() const
{
	return *Impl->OutputPath;
}

const TCHAR* FDatasmithSceneExporter::GetAssetsOutputPath() const
{
	return *Impl->AssetsOutputPath;
}
