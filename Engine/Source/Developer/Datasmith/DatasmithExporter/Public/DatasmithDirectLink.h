// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDatasmithDirectLinkExporterAPI, Log, All);


class IDatasmithScene;

namespace DirectLink
{
class FEndpoint;
}

class DATASMITHEXPORTER_API FDatasmithDirectLink
{
public:
	static int32 ValidateCommunicationSetup();
	static bool Shutdown();

public:
	FDatasmithDirectLink();

	bool InitializeForScene(const TSharedRef<IDatasmithScene>& Scene);
	bool UpdateScene(const TSharedRef<IDatasmithScene>& Scene);

	/**
	 * Close the initialized DirectLink source if any.
	 */
	void CloseCurrentSource();

	static TSharedRef<DirectLink::FEndpoint, ESPMode::ThreadSafe> GetEnpoint();
};
