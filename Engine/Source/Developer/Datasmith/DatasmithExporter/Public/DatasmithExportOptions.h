// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "DatasmithUtils.h"

/**
 * Never: it won't force the creation of a UV channel for lightmap, if one already exists it will be exported
 * IfNotPresent: if a channel already exists it will be used, if not a new automatic UV projection will be created
 * Always: it will create the UV channel overwritting existing data
 */
enum class EDSExportLightmapUV
{
	Never,
	IfNotPresent,
	Always
};

/**
 * OriginalFolder: keeps the textures (and its resized copies if needed) in the original folder
 * ExportFolder: copy the textures to the export path
 */
enum class EDSResizedTexturesPath
{
	OriginalFolder,
	ExportFolder
};

class DATASMITHEXPORTER_API FDatasmithExportOptions
{
public:

	/** Creation of automatic flatten projection of UVs to be used on lightmaps */
	static EDSExportLightmapUV LightmapUV;
	/** Policy to create GPU friendly texture sizes */
	static EDSResizeTextureMode ResizeTexturesMode;
	/** it sets where the resized textures will be stored */
	static EDSResizedTexturesPath PathTexturesMode;
	/** Maximum texture size allowed, this will be used in conjuction with ResizeTexturesMode */
	static int32 MaxTextureSize;
	static const int32 MaxUnrealSupportedTextureSize;

	/** Gamma of colors, usually 2.2 for Linear Workflow or 1.0 for classic workflow */
	static float ColorGamma;
};