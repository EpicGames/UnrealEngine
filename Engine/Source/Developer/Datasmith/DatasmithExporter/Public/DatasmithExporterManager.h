// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

#if IS_PROGRAM
#include "Templates/SharedPointer.h"

struct FDefaultDelegateUserPolicy;
template <typename DelegateSignature, typename UserPolicy>
class TDelegate;
typedef TDelegate<void(), FDefaultDelegateUserPolicy> FSimpleDelegate;
#endif // IS_PROGRAM

class DATASMITHEXPORTER_API FDatasmithExporterManager
{
public:
	struct FInitOptions
	{
		bool bSuppressLogs = true;
		bool bSaveLogToUserDir = true;
		bool bEnableMessaging = false;

		/**
		 * This setting will enable the datasmith exporter UI
		 * Enabling this will require the remote engine dir path to be specified
		 * It will set the engine loop in it's own thread so that the ui can run independently from the host program
		 *
		 * If your are not using the datasmith SDK or the C# SDK (Datasmith Facade) you will also need to add the fellowing line to your target.cs file
		 * GlobalDefinitions.Add("UE_EXTERNAL_PROFILING_ENABLED=0");
		 * Otherwise the engine path won't be properly initialized as these profilers require the engine path to be valid when the static variables are initialized
		*/
		bool bUseDatasmithExporterUI = false;
		const TCHAR* RemoteEngineDirPath = nullptr;
	};

	/**
	 * Initializes the Datasmith Exporter module with the default settings
	 *
	 * Needs to be called before starting any export or using any other features of the datasmith export.
	 * Must be called once
	 *
	 * @return True if the initialization was successful
	 */
	static bool Initialize();

	/**
	 * Initializes the Datasmith Exporter module.
	 * @param InitOptions The options to select the features of the datasmith sdk that should be activated
	 *
	 * Needs to be called before starting any export or using any other features of the datasmith export module.
	 * Must be called once
	 *
	 * @return True if the initialization was successful
	 */

	static bool Initialize(const FInitOptions& InitOptions);

	/**
	 * Shuts down the Datasmith Exporter module.
	 * Must be called when the process performing exports exits
	 */
	static void Shutdown();

	/**
	 * Run the unreal engine garbage collection on the proper thread
	 * Exporter code that want to run garbage collection should use this function instead of the standard engine function
	 * @return whether the Garbage Collection ran.
	 */
	static bool RunGarbageCollection();

#if IS_PROGRAM
	/**
	 * Enqueue a command into the game thread for execution
	 */
	static void PushCommandIntoGameThread(FSimpleDelegate&& Command, bool bWakeUpGameThread = false);

	/**
	 * Returns if the Exporter Manager was initialized with enabled network capabilities.
	 */
	static bool WasInitializedWithMessaging();

	/**
	 * Returns if the Exporter Manager is running its own game thread.
	 */
	static bool WasInitializedWithGameThread();

private:
	static bool bEngineInitialized;
	static bool bUseMessaging;
	static class FRunnableThread* GMainThreadAsRunnable;

	// SharedPtr because we want to forward declare the type
	static TSharedPtr<class FDatasmithGameThread> GDatasmithGameThread;
#endif // IS_PROGRAM
};

