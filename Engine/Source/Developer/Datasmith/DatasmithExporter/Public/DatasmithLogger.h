// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FDatasmithLoggerImpl;

/**
* Logger that holds the errors during the export process.
* The user plugin should display the list of errors at the end of the export.
*/
class DATASMITHEXPORTER_API FDatasmithLogger
{
public:
	FDatasmithLogger();
	virtual ~FDatasmithLogger();

	void AddGeneralError(const TCHAR* InError);
	int32 GetGeneralErrorsCount();
	const TCHAR* GetGeneralError(int32 Index) const;
	void ResetGeneralErrors();

	void AddTextureError(const TCHAR* InError);
	int32 GetTextureErrorsCount();
	const TCHAR* GetTextureError(int32 Index) const;
	void ResetTextureErrors();

	void AddMissingAssetError(const TCHAR* InError);
	int32 GetMissingAssetErrorsCount();
	const TCHAR* GetMissingAssetError(int32 Index) const;
	void ResetMissingAssetErrors();

protected:
	FDatasmithLoggerImpl* Impl;

};
