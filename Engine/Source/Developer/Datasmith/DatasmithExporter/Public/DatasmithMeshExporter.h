// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "DatasmithExportOptions.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class FDatasmithMesh;
class IDatasmithMeshElement;

class DATASMITHEXPORTER_API FDatasmithMeshExporter
{
public:
	FDatasmithMeshExporter();
	virtual ~FDatasmithMeshExporter();

	/**
	 * Exports the DatasmithMesh as a UObject.
	 *
	 * @param Filepath		The path where the resulting file will be written
	 * @param Filename		The name of the file to export to, without any path or extension
	 * @param Mesh	        The mesh to export
	 * @param CollisionMesh An optional collision mesh
	 * @param LightmapUV	The UV generation export option
	 *
	 * @return				A IDatasmithMeshElement that refers to the exported file
	 */
	TSharedPtr<IDatasmithMeshElement> ExportToUObject(const TCHAR* Filepath, const TCHAR* Filename, FDatasmithMesh& Mesh, FDatasmithMesh* CollisionMesh, EDSExportLightmapUV LightmapUV);

	/**
	 * Exports a FDatasmithMesh as a UObject and link it to the given IDatasmithMeshElementElement.
	 *
	 * @param MeshElement	The existing MeshElement for which we want to export a FDatasmithMesh, the name of the MeshElement will determine the name of the exported file.
	 * @param Filepath		The path where the resulting file will be written
	 * @param Mesh	        The mesh to export
	 * @param CollisionMesh An optional collision mesh
	 * @param LightmapUV	The UV generation export option
	 *
	 * @return				True if export was successful.
	 */
	bool ExportToUObject(TSharedPtr<IDatasmithMeshElement>& MeshElement, const TCHAR* Filepath, FDatasmithMesh& Mesh, FDatasmithMesh* CollisionMesh, EDSExportLightmapUV LightmapUV);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	bool ExportCloth(class FDatasmithCloth& Cloth, TSharedPtr<class IDatasmithClothElement>& ClothElement, const TCHAR* FilePath, const TCHAR* AssetsOutputPath) const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * @return The error that happened during the last export, if any
	 */
	FString GetLastError() const;

private:
	TUniquePtr<class FDatasmithMeshExporterImpl> Impl;
};
