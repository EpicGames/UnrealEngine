// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Interface that is called during export time to update the progression.
 * Implement for your specific DCC.
 */
class IDatasmithProgressManager
{
public:
	virtual ~IDatasmithProgressManager() {};

	/**
	 * Method called when progress is made during an export.
	 *
	 * @param InProgressRatio	The ratio of progress, between 0.f and 1.f
	 * @param InProgressString	Text describing the current task being performed
	 */
	virtual void ProgressEvent(float InProgressRatio, const TCHAR* InProgressString) = 0;
};
