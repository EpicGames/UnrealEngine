// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "DatasmithTypes.h"
#include "DatasmithUtils.h"
#include "IDatasmithSceneElements.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class FDatasmithLogger;
class FDatasmithSceneExporterImpl;
class IDatasmithProgressManager;
class IDatasmithScene;

/**
 * This is the export for a DatasmithScene. Call PreExport, then Export to finish the export process.
 */
class DATASMITHEXPORTER_API FDatasmithSceneExporter
{
public:
	FDatasmithSceneExporter();
	~FDatasmithSceneExporter();

	/**
	 * Indicates that we're starting the export process. Starts the export metrics.
	 */
	void PreExport();

	/**
	 * Exports the entire scene.
	 * It will create the scene file as well as the resized textures (in case of resize is enabled).
	 *
	 * @param bCleanupUnusedElements Remove unused meshes, textures and materials before exporting
	 */
	void Export( TSharedRef< IDatasmithScene > DatasmithScene, bool bCleanupUnusedElements = true );

	/** Resets all the settings on the scene */
	UE_DEPRECATED(5.1, "This function was selectively reseting the export state and was preserving the export paths. We should now directly set the fields we want to change instead.")
	void Reset();

	/** Sets the progress manager for visual feedback on exporting */
	void SetProgressManager( const TSharedPtr< IDatasmithProgressManager >& InProgressManager );

	/** Sets the logger to store the summary of the export process */
	void SetLogger( const TSharedPtr< FDatasmithLogger >& InLogger );

	/** Sets the name of the scene to export. The resulting file and folder will use this name. */
    void SetName(const TCHAR* InName);

	/** Gets the name of the scene to export. */
	const TCHAR* GetName() const;

	/**
	 * Sets the output path to where this scene will be exported.
	 */
	void SetOutputPath(const TCHAR* InOutputPath);
	const TCHAR* GetOutputPath() const;

	/**
	 * Gets the path to the assets output folder. This is where we output the mesh files, textures, etc.
	 */
	const TCHAR* GetAssetsOutputPath() const;

private:
	TUniquePtr< FDatasmithSceneExporterImpl > Impl;
};
