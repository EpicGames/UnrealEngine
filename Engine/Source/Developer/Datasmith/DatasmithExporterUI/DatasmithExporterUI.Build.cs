// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool.Rules
{
	[SupportedPlatforms("Win64", "Mac", "Linux")]
	public class DatasmithExporterUI : ModuleRules
	{
		public DatasmithExporterUI(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DatasmithCore",
					"DatasmithExporter",
					"DesktopPlatform",
					"DirectLink",
					"InputCore",
					"Slate",
					"SlateCore",
				});
		}
	}
}