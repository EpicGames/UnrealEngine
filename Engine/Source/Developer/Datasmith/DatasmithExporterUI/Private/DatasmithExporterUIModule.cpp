// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithExporterUIModule.h"

#include "DatasmithExporterManager.h"
#include "DirectLinkUI.h"

#include <atomic>

IMPLEMENT_MODULE( FDatasmithExporterUIModule, DatasmithExporterUI );

std::atomic<class FDatasmithExporterUIModule*> GExporterUIModule(nullptr);

IDatasmithExporterUIModule* IDatasmithExporterUIModule::Get()
{
	return GExporterUIModule;
}

void FDatasmithExporterUIModule::StartupModule()
{
	GExporterUIModule = this;

#if IS_PROGRAM
	if ( FDatasmithExporterManager::WasInitializedWithMessaging() )
	{
		DirectLinkUI = MakeUnique<FDirectLinkUI>();
	}
#endif
}

IDirectLinkUI* FDatasmithExporterUIModule::GetDirectLinkExporterUI() const
{
	return DirectLinkUI.Get();
}

