// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDatasmithExporterUIModule.h"
#include "IDirectLinkUI.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FDatasmithExporterUIModule final : public IModuleInterface, public IDatasmithExporterUIModule
{
public:
	virtual bool SupportsDynamicReloading()
	{
		return false;
	}

	virtual void StartupModule() override;

	virtual IDirectLinkUI* GetDirectLinkExporterUI() const override;

private:
	TUniquePtr<IDirectLinkUI> DirectLinkUI;
};