// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkUI.h"

#include "DatasmithDirectLink.h"
#include "DatasmithExporterManager.h"
#include "DirectLinkEndpoint.h"
#include "Widgets/SDirectLinkStreamManager.h"

#include "Containers/Queue.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Framework/Application/SWindowTitleBar.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "Styling/CoreStyle.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "DirectLinkUI"

namespace DirectLinkUIUtils
{
	FString GetConfigPath()
	{
		return FPaths::Combine( FPaths::GeneratedConfigDir(), TEXT("DirectLinkExporter") ).Append( TEXT(".ini") );;
	}

	FString GetConfigCacheDirectorySectionAndValue( bool bInDefaultCacheDir )
	{
		return bInDefaultCacheDir ? TEXT("DLDefaultCacheFolder") : TEXT("DLCacheFolder");
	}

	EHorizontalAlignment GetWindowTitleAlignement()
	{
		EWindowTitleAlignment::Type TitleAlignment = FSlateApplicationBase::Get().GetPlatformApplication()->GetWindowTitleAlignment();
		EHorizontalAlignment TitleContentAlignment;

		if ( TitleAlignment == EWindowTitleAlignment::Left )
		{
			TitleContentAlignment = HAlign_Left;
		}
		else if ( TitleAlignment == EWindowTitleAlignment::Center )
		{
			TitleContentAlignment = HAlign_Center;
		}
		else
		{
			TitleContentAlignment = HAlign_Right;
		}
		return TitleContentAlignment;
	}
}

FVector2D FDirectLinkUI::StreamWindowDefaultSize = FVector2D(640, 480);

FDirectLinkUI::FDirectLinkUI()
	: StreamWindowSize(StreamWindowDefaultSize)
{
	const FString ConfigPath = DirectLinkUIUtils::GetConfigPath();
	const FString DirectLinkCacheSectionAndValue = DirectLinkUIUtils::GetConfigCacheDirectorySectionAndValue( false );
	const FString DirectLinkDefaultCacheSectionAndValue = DirectLinkUIUtils::GetConfigCacheDirectorySectionAndValue( true );

	FScopeLock Lock( &CriticalSectionCacheDirectory );

	// Verify and get default cache directory
	GConfig->GetString(*DirectLinkDefaultCacheSectionAndValue, *DirectLinkDefaultCacheSectionAndValue, DefaultDirectLinkCacheDirectory, ConfigPath);

	if (DefaultDirectLinkCacheDirectory.IsEmpty() || !FPaths::DirectoryExists(DefaultDirectLinkCacheDirectory))
	{
		if (DefaultDirectLinkCacheDirectory.IsEmpty() || !IFileManager::Get().MakeDirectory(*DefaultDirectLinkCacheDirectory, true))
		{
			DefaultDirectLinkCacheDirectory = FPaths::Combine(FPlatformProcess::UserTempDir(), TEXT("DLExporter"));
			if (!FPaths::DirectoryExists(DefaultDirectLinkCacheDirectory))
			{
				ensure(IFileManager::Get().MakeDirectory(*DefaultDirectLinkCacheDirectory, true));
			}
		}

	}

	GConfig->SetString(*DirectLinkDefaultCacheSectionAndValue, *DirectLinkDefaultCacheSectionAndValue, *DefaultDirectLinkCacheDirectory, ConfigPath);

	// Verify and get current cache directory
	GConfig->GetString(*DirectLinkCacheSectionAndValue, *DirectLinkCacheSectionAndValue, DirectLinkCacheDirectory, ConfigPath);

	if (DirectLinkCacheDirectory.IsEmpty() || !FPaths::DirectoryExists(DirectLinkCacheDirectory))
	{
		if (DirectLinkCacheDirectory.IsEmpty() || !IFileManager::Get().MakeDirectory(*DirectLinkCacheDirectory, true))
		{
			DirectLinkCacheDirectory = DefaultDirectLinkCacheDirectory;
		}

	}

	GConfig->SetString(*DirectLinkCacheSectionAndValue, *DirectLinkCacheSectionAndValue, *DirectLinkCacheDirectory, ConfigPath);
}

void FDirectLinkUI::SetStreamWindowCenter( int InCenterX, int InCenterY )
{
	StreamWindowCenterSet = true;
	StreamWindowPosition = FVector2D(InCenterX, InCenterY) - (StreamWindowSize / 2);
}

void FDirectLinkUI::OpenDirectLinkStreamWindow()
{
	FSimpleDelegate RunOnUIThread;

	TSharedRef<DirectLink::FEndpoint, ESPMode::ThreadSafe> Enpoint = FDatasmithDirectLink::GetEnpoint();

	RunOnUIThread.BindLambda( [this, Enpoint]()
	{
		if ( TSharedPtr<SWindow> OpennWindow = DirectLinkWindow.Pin() )
		{
			// Cancel the minimize
			OpennWindow->BringToFront();

			OpennWindow->HACK_ForceToFront();
			OpennWindow->FlashWindow();
		}
		else
		{
			EAutoCenter AutoCenterValue = StreamWindowClosedBefore ? (EAutoCenter::None) : (StreamWindowCenterSet ? EAutoCenter::None : EAutoCenter::PrimaryWorkArea);

			// This window setup might be an issue on mac where users often expect os borders on their window
			TSharedRef<SWindow> Window = SNew( SWindow )
				.CreateTitleBar( false )
				.ClientSize( StreamWindowDefaultSize )
				.AutoCenter( AutoCenterValue )
				.SizingRule( ESizingRule::UserSized )
				.FocusWhenFirstShown( true )
				.Title( LOCTEXT("DirectlinkStreamManagerWindowTitle", "Datasmith Direct Link Connection Status") );

			if ( AutoCenterValue == EAutoCenter::None )
			{
				Window->ReshapeWindow( StreamWindowPosition, StreamWindowSize );
			}
			
			TSharedRef<SWindowTitleBar> WindowTitleBar = SNew( SWindowTitleBar, Window, nullptr, DirectLinkUIUtils::GetWindowTitleAlignement() )
				.Visibility( EVisibility::Visible )
				.ShowAppIcon( false );

			Window->SetTitleBar( WindowTitleBar );

			Window->SetContent(
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						WindowTitleBar
					]
					+ SVerticalBox::Slot()
					.FillHeight( 1.f )
					[
						SNew( SBorder )
						.BorderImage( FCoreStyle::Get().GetBrush( "ToolPanel.GroupBorder" ) )
						[
							SNew( SDirectLinkStreamManager, Enpoint )
							// There can only be one window that change this value (no locks required here)
							.DefaultCacheDirectory( DirectLinkCacheDirectory )
							// The slate application should always be close before this module.
							.OnCacheDirectoryChanged_Raw( this, &FDirectLinkUI::OnCacheDirectoryChanged )
							.OnCacheDirectoryReset_Raw( this, &FDirectLinkUI::OnCacheDirectoryReset )
						]
					]
				);
			Window->SetOnWindowClosed( FOnWindowClosed::CreateRaw(this, &FDirectLinkUI::WindowClosed) );
			DirectLinkWindow = Window;
			FSlateApplication::Get().AddWindow( Window, true );
			Window->HACK_ForceToFront();
		}
	});

#if IS_PROGRAM
	FDatasmithExporterManager::PushCommandIntoGameThread(MoveTemp(RunOnUIThread));
#else
	RunOnUIThread.Execute();
#endif
}

void FDirectLinkUI::WindowClosed( const TSharedRef<SWindow>& WindowArg )
{
	StreamWindowClosedBefore = true;

	StreamWindowSize = WindowArg->GetSizeInScreen();
	StreamWindowPosition = WindowArg->GetPositionInScreen();
}

const TCHAR* FDirectLinkUI::GetDirectLinkCacheDirectory()
{
	// This function should always be called from one thread only
	static const uint32 FirstCallThreadId = FPlatformTLS::GetCurrentThreadId();
	check(FirstCallThreadId == FPlatformTLS::GetCurrentThreadId());

	{
		FScopeLock ReadLock( &CriticalSectionCacheDirectory );
		LastReturnedCacheDirectory = DirectLinkCacheDirectory;
	}

	return *LastReturnedCacheDirectory;
}

void FDirectLinkUI::OnCacheDirectoryChanged(const FString& InNewCacheDirectory)
{
	{
		FScopeLock ReadLock( &CriticalSectionCacheDirectory );
		DirectLinkCacheDirectory = InNewCacheDirectory;
	}

	SaveCacheDirectory( InNewCacheDirectory, false );

}

FString FDirectLinkUI::OnCacheDirectoryReset()
{
	{
		FScopeLock ReadLock( &CriticalSectionCacheDirectory );
		DirectLinkCacheDirectory = DefaultDirectLinkCacheDirectory;
	}

	SaveCacheDirectory( DefaultDirectLinkCacheDirectory, false );

	return DefaultDirectLinkCacheDirectory;
}

void FDirectLinkUI::SaveCacheDirectory( const FString& InCacheDir, bool bInDefaultCacheDir )
{
	FString ConfigPath = DirectLinkUIUtils::GetConfigPath();
	FString DirectLinkCacheSectionAndValue = DirectLinkUIUtils::GetConfigCacheDirectorySectionAndValue( bInDefaultCacheDir );

	// Save to config file
	GConfig->SetString( *DirectLinkCacheSectionAndValue, *DirectLinkCacheSectionAndValue, *InCacheDir, ConfigPath );
}

#undef LOCTEXT_NAMESPACE
