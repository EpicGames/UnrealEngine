// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDirectLinkUI.h"

#include "Containers/UnrealString.h"
#include "Math/Vector2D.h"
#include "HAL/CriticalSection.h"
#include "Templates/SharedPointer.h"

class SWindow;

class FDirectLinkUI : public IDirectLinkUI
{
public:

	FDirectLinkUI();

	virtual void SetStreamWindowCenter( int InCenterX, int InCenterY ) override;

	virtual void OpenDirectLinkStreamWindow() override;
	virtual const TCHAR* GetDirectLinkCacheDirectory() override;

private: 
	void OnCacheDirectoryChanged(const FString& InNewCacheDirectory);
	FString OnCacheDirectoryReset();

	void SaveCacheDirectory( const FString& InCacheDir, bool bInDefaultCacheDir );

	void WindowClosed( const TSharedRef<SWindow>& WindowArg );

	// To accessed from the game thread only
	TWeakPtr<SWindow> DirectLinkWindow;
	
	FCriticalSection CriticalSectionCacheDirectory;
	FString DirectLinkCacheDirectory;
	FString DefaultDirectLinkCacheDirectory;

	// Used to protect the caller to GetDirectLinkCacheDirectory from a pontential race condition
	FString LastReturnedCacheDirectory;

	bool StreamWindowCenterSet = false;
	bool StreamWindowClosedBefore = false;
	static FVector2D StreamWindowDefaultSize;
	FVector2D StreamWindowSize;
	FVector2D StreamWindowPosition;
};