// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDirectLinkStreamManager.h"

#include "DatasmithExporterManager.h"
#include "DirectLinkEndpoint.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "IDesktopPlatform.h"
#include "Styling/CoreStyle.h"
#include "UObject/NoExportTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SDirectLinkStreamManager"

namespace SDirectLinkStreamManagerUtils
{
	const FName SourceColumnId("Source");
	const FName DestinationColumnId("Destination");
}

struct FEndpointData
{
	FEndpointData(const FString& InEndpointName, const FString& InUserName, const FString& InExecutableName, const FString& InComputerName)
		: EndpointName( InEndpointName )
		, UserName( InUserName )
		, ExecutableName( InExecutableName )
		, ComputerName( InComputerName )
	{
	}

	FString EndpointName;
	FString UserName;
	FString ExecutableName;
	FString ComputerName;

	FText GenerateTooltipText() const;
};

FText FEndpointData::GenerateTooltipText() const
{
	FText ToolTipNonFormated = LOCTEXT("EndpointTooltip", "User : {0}\nComputer Name : {1}\nProgram Name: {2}\nEndpoint Name : {3}");
	return FText::FormatOrdered( ToolTipNonFormated,
			FText::FromString( UserName ),
			FText::FromString( ComputerName ),
			FText::FromString( ExecutableName ),
			FText::FromString( EndpointName )
		);
}


struct FSourceData
{
	FSourceData(const FString& InName, const FGuid& InGuid, const TSharedRef<FEndpointData>& InEndpointData)
		: Name( InName )
		, Guid( InGuid )
		, EndpointData( InEndpointData )
	{
	}

	FString Name;
	FGuid Guid;

	TSharedRef<FEndpointData> EndpointData;
};

struct FDestinationData
{
	FDestinationData(const FString& InName, const FGuid& InGuid, const TSharedRef<FEndpointData>& InEndpointData)
		: Name( InName )
		, Guid( InGuid )
		, EndpointData( InEndpointData )
	{
	}

	FString Name;
	FGuid Guid;

	TSharedRef<FEndpointData> EndpointData;
};

struct FStreamData
{
	FStreamData(bool bInIsActive, const TSharedRef<FSourceData>& InSource, const TSharedRef<FDestinationData>& InDestination)
		: bIsActive( bInIsActive )
		, SourceColumnLabel( InSource->Name )
		, DestinationColumnLabel()
		, Source( InSource )
		, Destination( InDestination )
	{
		uint32 StringLenght = Destination->EndpointData->EndpointName.Len();
		// " : "
		StringLenght +=	3;
		StringLenght += Destination->Name.Len();
		DestinationColumnLabel.Reserve( StringLenght );

		DestinationColumnLabel.Append( Destination->EndpointData->EndpointName );
		DestinationColumnLabel.Append( " : " );
		DestinationColumnLabel.Append( Destination->Name );
	}

	bool bIsActive;

	FString SourceColumnLabel;
	FString DestinationColumnLabel;

	TSharedRef<FSourceData> Source;
	TSharedRef<FDestinationData> Destination;
};

class SDirectLinkStreamManagerRow : public SMultiColumnTableRow<TSharedRef<FStreamData>>
	{
	public:
		SLATE_BEGIN_ARGS(SDirectLinkStreamManagerRow)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<FStreamData> InItem, const TSharedRef<STableViewBase>& InOwner);

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	private:
		TWeakPtr<FStreamData> ItemWeakPtr;
	};

void SDirectLinkStreamManagerRow::Construct(const FArguments& InArgs, TSharedRef<FStreamData> InItem, const TSharedRef<STableViewBase>& InOwner)
{
	ItemWeakPtr = InItem;

	FSuperRowType::FArguments Args = FSuperRowType::FArguments();
	FSuperRowType::Construct( Args, InOwner );
}


TSharedRef<SWidget> SDirectLinkStreamManagerRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if ( TSharedPtr<FStreamData> Item = ItemWeakPtr.Pin() )
	{
		if ( ColumnName == SDirectLinkStreamManagerUtils::SourceColumnId )
		{
			return SNew(STextBlock)
				.Text( FText::FromString( Item->Source->Name ) )
				.ToolTipText( Item->Source->EndpointData->GenerateTooltipText() );
		}
		else if ( ColumnName == SDirectLinkStreamManagerUtils::DestinationColumnId )
		{
			// We don't expect that data to change over the lifetime of the stream
			return SNew(STextBlock)
				.Text( FText::FromString( Item->DestinationColumnLabel ) )
				.ToolTipText( Item->Destination->EndpointData->GenerateTooltipText() );
		}

		checkNoEntry();
	}

	return SNullWidget::NullWidget;
}


class FEndpointObserver : public DirectLink::IEndpointObserver
{
public:
	TWeakPtr<SDirectLinkStreamManager> WidgetToUpdate;

	virtual void OnStateChanged(const DirectLink::FRawInfo& RawInfo) override
	{
		FSimpleDelegate RunOnMainThread;
		RunOnMainThread.BindLambda( [this, RawInfo]()
			{
				if ( TSharedPtr<SDirectLinkStreamManager> StreamManager = WidgetToUpdate.Pin() )
				{
					StreamManager->UpdateData( RawInfo );
				}
			} );

#if IS_PROGRAM
	FDatasmithExporterManager::PushCommandIntoGameThread( MoveTemp(RunOnMainThread) );
#else
	// Unsupported for now
	checkNoEntry();
#endif
	}
};

void SDirectLinkStreamManager::Construct(const FArguments& InArgs, const TSharedRef<DirectLink::FEndpoint, ESPMode::ThreadSafe>& InEndpoint)
{
	bShowingAdavancedSetting = false;
	SortedColumn = SDirectLinkStreamManagerUtils::SourceColumnId;
	SortMode = EColumnSortMode::Ascending;

	DirectLinkCacheDirectory = InArgs._DefaultCacheDirectory;
	OnCacheDirectoryChanged = InArgs._OnCacheDirectoryChanged;
	OnCacheDirectoryReset = InArgs._OnCacheDirectoryReset;

	TSharedRef<SHeaderRow> HeaderRow = SNew( SHeaderRow )
		// Source
		+ SHeaderRow::Column( SDirectLinkStreamManagerUtils::SourceColumnId )
		.DefaultLabel( LOCTEXT("SourceolumnLabel", "Source") )
		.SortMode( this, &SDirectLinkStreamManager::GetColumnSortMode, SDirectLinkStreamManagerUtils::SourceColumnId )
		.OnSort( this, &SDirectLinkStreamManager::OnColumnSortModeChanged )
		// Destination
		+ SHeaderRow::Column( SDirectLinkStreamManagerUtils::DestinationColumnId )
		.DefaultLabel( LOCTEXT("DestinationColumnLabel", "Destination") )
		.SortMode( this, &SDirectLinkStreamManager::GetColumnSortMode, SDirectLinkStreamManagerUtils::DestinationColumnId )
		.OnSort( this, &SDirectLinkStreamManager::OnColumnSortModeChanged );

	ChildSlot
	[
		SNew( SVerticalBox )
		// No connection hint
		+ SVerticalBox::Slot()
		.Padding( 2.f )
		.FillHeight( TAttribute<float>( this, &SDirectLinkStreamManager::GetNoConnectionHintFillHeight ) )
		.VAlign( VAlign_Center )
		[
				SNew( STextBlock )
				.Visibility( this, &SDirectLinkStreamManager::GetNoConnectionHintVisibility )
				.Text( LOCTEXT("ConnectionHintText", "No connection found.\n\nWaiting for a Datasmith Direct Link connection to be established.\n\nPlease start an application supporting Datasmith Direct Link such as Twinmotion or Unreal Engine.") )
				.AutoWrapText( true )
				.Justification( ETextJustify::Center )
				.Font( FCoreStyle::GetDefaultFontStyle( "Regular", 13 ) )
		]

		// Connection view
		+ SVerticalBox::Slot()
		.Padding( 2.f )
		.VAlign( VAlign_Fill )
		[
			SAssignNew( ConnectionsView, SListView<TSharedRef<FStreamData>> )
			.Visibility( this, &SDirectLinkStreamManager::GetConnectionViewVisibility )
			.ListItemsSource( &Streams )
			.OnGenerateRow( this, &SDirectLinkStreamManager::OnGenerateRow )
			.SelectionMode( ESelectionMode::None )
			.HeaderRow( HeaderRow )
		]

		// Drop shadow
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign( VAlign_Bottom )
		[
			SNew( SImage )
			.Visibility( this, &SDirectLinkStreamManager::GetAdavancedSettingVisibility )
			// Use the drop shadow of a scroll box to add a bit of depth to the setting
			.Image( &(FCoreStyle::Get().GetWidgetStyle<FScrollBoxStyle>("ScrollBox").TopShadowBrush) )
		]

		// Cache directory
		+ SVerticalBox::Slot()
		.Padding( 2.f )
		.AutoHeight()
		.VAlign( VAlign_Bottom )
		[
			SNew( SHorizontalBox )
			.Visibility( this, &SDirectLinkStreamManager::GetAdavancedSettingVisibility )
			+ SHorizontalBox::Slot()
			// More padding to the right to separate the label from the content
			.Padding( 2.f, 2.f, 4.f, 2.f )
			.AutoWidth()
			.HAlign( HAlign_Left )
			[
				SNew( SVerticalBox )
				+ SVerticalBox::Slot()
				.VAlign( VAlign_Center )
				[
					SNew( STextBlock )
					.Text( LOCTEXT("CacheDirectoryLabel", "Cache Directory:") )
					.ToolTipText( LOCTEXT("CacheDirectoryLabelToolTip","Direct Link will use this directory to store temporary files") )
				]
			]
			+ SHorizontalBox::Slot()
			.Padding( 2.f )
			.HAlign( HAlign_Right )
			[
				SNew( SVerticalBox )
				+ SVerticalBox::Slot()
				.VAlign( VAlign_Center )
				[
					SNew( SEditableText )
					.IsReadOnly( true )
					.Text( this, &SDirectLinkStreamManager::GetCacheDirectory )
					.ToolTipText( this, &SDirectLinkStreamManager::GetCacheDirectory )
				]
			]
			+ SHorizontalBox::Slot()
			.Padding( 2.f )
			.AutoWidth()
			.HAlign( HAlign_Right )
			[
				SNew( SButton )
				.OnClicked( this, &SDirectLinkStreamManager::OnResetCacheDirectoryClicked )
				.ToolTipText( LOCTEXT("ResetCacheDirectory_Tooltip", "Reset cache directory") )
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.VAlign( VAlign_Center )
					[
						SNew( STextBlock )
						.Text( LOCTEXT("ResetCacheDirectory_Label", "Reset") )
					]
				]
			]
			+ SHorizontalBox::Slot()
			.Padding( 2.f )
			.AutoWidth()
			.HAlign( HAlign_Right )
			[
				SNew( SButton )
				.OnClicked( this, &SDirectLinkStreamManager::OnChangeCacheDirectoryClicked )
				.ToolTipText( LOCTEXT("ChangeCacheDirectory_Tooltip", "Browse for another folder") )
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.VAlign( VAlign_Center )
					[
						SNew( STextBlock )
						.Text( LOCTEXT("...", "...") )
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding( 2.f )
		.AutoHeight()
		.VAlign( VAlign_Bottom )
		[
			SAssignNew( ShowAdavancedSettingButton, SButton )
			.ButtonStyle( &FCoreStyle::Get().GetWidgetStyle<FButtonStyle>( "NoBorder" ) )
			.HAlign( HAlign_Center )
			.ContentPadding( 2.f )
			.OnClicked( this, &SDirectLinkStreamManager::OnShowAdavancedSettingClicked )
			.ToolTipText( this, &SDirectLinkStreamManager::GetShowAdavancedSettingToolTipText )
			[
				SAssignNew( ShowAdavancedSettingImage, SImage )
				.Image( &(FCoreStyle::Get().GetWidgetStyle<FHeaderRowStyle>("TableView.Header").ColumnStyle.MenuDropdownImage) )
			]
		]
	];

	ShowAdavancedSettingImage->SetRenderTransformPivot( FVector2D(0.5f, 0.5f) );


	Endpoint = InEndpoint;
	Observer = MakeShared<FEndpointObserver>();
	Observer->WidgetToUpdate = StaticCastSharedRef<SDirectLinkStreamManager>(AsShared());
	UpdateData( InEndpoint->GetRawInfoCopy() );
	InEndpoint->AddEndpointObserver( Observer.Get() );
}

SDirectLinkStreamManager::~SDirectLinkStreamManager()
{
	Endpoint->RemoveEndpointObserver( Observer.Get() );
}

void SDirectLinkStreamManager::UpdateData(const DirectLink::FRawInfo& RawInfo)
{
	using namespace DirectLink;

	TMap<FGuid,TSharedRef<FDestinationData>> DestinationsMap;
	TMap<FGuid, TSharedRef<FSourceData>> SourcesMap;

	// Grab the sources
	{
		const FRawInfo::FEndpointInfo& EndpointInfo= RawInfo.EndpointsInfo.FindChecked( RawInfo.ThisEndpointAddress );

		TSharedRef<FEndpointData> EndpointData = MakeShared<FEndpointData>( EndpointInfo.Name,
			EndpointInfo.UserName,
			EndpointInfo.ExecutableName,
			EndpointInfo.ComputerName
			);

		// Grab all the sources
		SourcesMap.Reserve( EndpointInfo.Sources.Num() );
		for ( const FRawInfo::FDataPointId& NamedId : EndpointInfo.Sources )
		{
			TSharedRef<FSourceData> Source = MakeShared<FSourceData>( NamedId.Name,
				NamedId.Id,
				EndpointData
				);
			SourcesMap.Add( NamedId.Id, MoveTemp( Source ) );
		}
	}


	// Grab all the potential destinations
	for ( const TPair<FMessageAddress, FRawInfo::FEndpointInfo>& EnpointsInfoPair : RawInfo.EndpointsInfo )
	{
		const FRawInfo::FEndpointInfo& EndpointInfo = EnpointsInfoPair.Value;
		TSharedRef<FEndpointData> EndpointData = MakeShared<FEndpointData>( EndpointInfo.Name,
			EndpointInfo.UserName,
			EndpointInfo.ExecutableName,
			EndpointInfo.ComputerName
			);

		DestinationsMap.Reserve( EndpointInfo.Destinations.Num() );
		for (const FRawInfo::FDataPointId& NamedId : EndpointInfo.Destinations)
		{
			TSharedRef<FDestinationData> Destination = MakeShared<FDestinationData>( NamedId.Name,
				NamedId.Id,
				EndpointData
				);
			DestinationsMap.Add( NamedId.Id , MoveTemp( Destination ) );
		}
	}


	bool bDirtyStreamList = false;
	// Remove the already connected destinations and create the connection list
	for ( const FRawInfo::FStreamInfo& StreamInfo : RawInfo.StreamsInfo )
	{
		TSharedPtr<FDestinationData> DestinationData;
		const bool bIsActive = StreamInfo.ConnectionState == EStreamConnectionState::Active;

		if ( bIsActive )
		{
			if ( DestinationsMap.Contains( StreamInfo.Destination ) )
			{
				DestinationData = DestinationsMap.FindAndRemoveChecked( StreamInfo.Destination );
			}
		}

		FStreamID StreamId( StreamInfo.Source, StreamInfo.Destination );
		uint32 StreamIdHash = GetTypeHash( StreamId );
		if ( TSharedRef<FStreamData>* StreamDataPtr = StreamMap.FindByHash( StreamIdHash, StreamId ) )
		{
			TSharedRef<FStreamData>& StreamData = *StreamDataPtr;

			// Made with the assumption that the source and destionation data can't change
			if ( StreamData->bIsActive != bIsActive )
			{
				StreamData->bIsActive = bIsActive;
				bDirtyStreamList = true;
			}
		}
		else if ( bIsActive )
		{
			TSharedRef<FSourceData>* SourceData = SourcesMap.Find( StreamInfo.Source );
			if ( DestinationData.IsValid() && SourceData != nullptr )
			{
				TSharedRef<FStreamData> StreamData = MakeShared<FStreamData>( bIsActive, *SourceData, DestinationData.ToSharedRef() );
				StreamMap.AddByHash( StreamIdHash, StreamId, StreamData );
				bDirtyStreamList = true;
			}
		}
	}

	if ( bDirtyStreamList )
	{
		StreamMap.GenerateValueArray( Streams );
		for ( int32 Index = 0; Index < Streams.Num(); ++Index )
		{
			TSharedRef<FStreamData>& Stream = Streams[Index];
			if ( !Stream->bIsActive )
			{
				Streams.RemoveAtSwap( Index, EAllowShrinking::No );
				--Index;
			}
		}

		Streams.Shrink();
		SortStreamList();
	}
}

TSharedRef<ITableRow> SDirectLinkStreamManager::OnGenerateRow(TSharedRef<FStreamData> Item, const TSharedRef<STableViewBase>& Owner) const
{
	return SNew(SDirectLinkStreamManagerRow, Item, Owner);
}

EColumnSortMode::Type SDirectLinkStreamManager::GetColumnSortMode(const FName InColumnId) const
{
	if ( SortedColumn == InColumnId )
	{
		return SortMode;
	}

	return EColumnSortMode::None;
}

void SDirectLinkStreamManager::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortedColumn = ColumnId;
	SortMode = InSortMode;
	SortStreamList();
}

void SDirectLinkStreamManager::SortStreamList()
{
	if ( SortMode != EColumnSortMode::None )
	{
		const bool bAscending = EColumnSortMode::Ascending ==  SortMode;
		if ( SortedColumn == SDirectLinkStreamManagerUtils::SourceColumnId )
		{
			Streams.StableSort( [bAscending](const TSharedRef<FStreamData>& First, const TSharedRef<FStreamData>& Second)
				{
					return First->Source->Name < Second->Source->Name == bAscending;
				} );
		}
		else if ( SortedColumn == SDirectLinkStreamManagerUtils::DestinationColumnId )
		{
			Streams.StableSort( [bAscending](const TSharedRef<FStreamData>& First, const TSharedRef<FStreamData>& Second)
				{
					return First->DestinationColumnLabel < Second->DestinationColumnLabel == bAscending;
				} );
		}
		else
		{
			checkNoEntry();
		}
	}

	ConnectionsView->RequestListRefresh();
}

FText SDirectLinkStreamManager::GetCacheDirectory() const
{
	return FText::FromString( DirectLinkCacheDirectory );
}

FReply SDirectLinkStreamManager::OnResetCacheDirectoryClicked()
{
	if( OnCacheDirectoryReset.IsBound() )
	{
		DirectLinkCacheDirectory = OnCacheDirectoryReset.Execute();
	}

	return FReply::Handled();
}

FReply SDirectLinkStreamManager::OnChangeCacheDirectoryClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	FString NewFolderSelected;
	bool bOpened = DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs( AsShared() ) ,
			LOCTEXT( "ChoseCacheDirectoreDialog", "Cache Directory" ).ToString(),
			DirectLinkCacheDirectory,
			NewFolderSelected
		);

	if ( bOpened )
	{
		DirectLinkCacheDirectory = NewFolderSelected;
		OnCacheDirectoryChanged.ExecuteIfBound( DirectLinkCacheDirectory );
	}

	return FReply::Handled();
}

EVisibility SDirectLinkStreamManager::GetNoConnectionHintVisibility() const
{
	return Streams.Num() == 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

float SDirectLinkStreamManager::GetNoConnectionHintFillHeight() const
{
	return Streams.Num() == 0 ? 1.f : 0.f;
}

EVisibility SDirectLinkStreamManager::GetConnectionViewVisibility() const
{
	return Streams.Num() != 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDirectLinkStreamManager::GetAdavancedSettingVisibility() const
{
	if ( bShowingAdavancedSetting )
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FReply SDirectLinkStreamManager::OnShowAdavancedSettingClicked()
{
	bShowingAdavancedSetting = !bShowingAdavancedSetting;

	if ( bShowingAdavancedSetting )
	{
		ShowAdavancedSettingImage->SetRenderTransform( FSlateRenderTransform( FScale2D(1.f,-1.f) ) );

	}
	else
	{
		ShowAdavancedSettingImage->SetRenderTransform( FSlateRenderTransform( FScale2D(1.f, 1.f) ) );
	}

	return FReply::Handled();
}

FText SDirectLinkStreamManager::GetShowAdavancedSettingToolTipText() const
{
	if ( bShowingAdavancedSetting )
	{
		return LOCTEXT("HideAdavancedSetting", "Hide setting");
	}

	return LOCTEXT("ShowAdavancedSetting","Show setting");
}

#undef LOCTEXT_NAMESPACE
