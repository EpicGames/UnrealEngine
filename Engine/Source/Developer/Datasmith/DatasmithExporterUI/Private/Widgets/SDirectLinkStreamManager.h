// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"

namespace DirectLink
{
	class FEndpoint;
	struct FRawInfo;
}

class FEndpointObserver;
class FEndpointObserver;
class ITableRow;
class STableViewBase;
struct FDestinationData;
struct FSourceData;
struct FStreamData;

template <typename ItemType>
class SListView;


class SDirectLinkStreamManager : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnCacheDirectoryChanged, const FString& /** NewDirectory */ );
	DECLARE_DELEGATE_RetVal( FString, FOnCacheDirectoryReset );

	SLATE_BEGIN_ARGS(SDirectLinkStreamManager) {}
		SLATE_EVENT( FOnCacheDirectoryChanged, OnCacheDirectoryChanged )
		SLATE_EVENT( FOnCacheDirectoryReset, OnCacheDirectoryReset )
		SLATE_ARGUMENT( FString, DefaultCacheDirectory )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<DirectLink::FEndpoint, ESPMode::ThreadSafe>& InEndpoint);

	virtual ~SDirectLinkStreamManager();

	void UpdateData(const DirectLink::FRawInfo& RawInfo);
private:

	using FSourceId = FGuid;
	using FDestinationId = FGuid;
	using FStreamID = TPair<FSourceId,FDestinationId>;

	TSharedRef<ITableRow> OnGenerateRow(TSharedRef<FStreamData> Item, const TSharedRef<STableViewBase>& Owner) const;

	EColumnSortMode::Type GetColumnSortMode(const FName InColumnId) const;

	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	void SortStreamList();

	FText GetCacheDirectory() const;

	FReply OnChangeCacheDirectoryClicked();
	FReply OnResetCacheDirectoryClicked();

	EVisibility GetNoConnectionHintVisibility() const;

	float GetNoConnectionHintFillHeight() const;

	EVisibility GetConnectionViewVisibility() const;

	EVisibility GetAdavancedSettingVisibility() const;

	FReply OnShowAdavancedSettingClicked();

	FText GetShowAdavancedSettingToolTipText() const;


	TSharedPtr<SListView<TSharedRef<FStreamData>>> ConnectionsView;

	// The list of streams connection
	TArray<TSharedRef<FStreamData>> Streams;

	// A map to update the stream data quickly
	TMap<FStreamID, TSharedRef<FStreamData>> StreamMap;

	// The endpoint managed by this widget
	TSharedPtr<DirectLink::FEndpoint, ESPMode::ThreadSafe> Endpoint;

	// The registered observer to the endpoint
	TSharedPtr<FEndpointObserver> Observer;

	// The currently sorted column
	FName SortedColumn;

	// The current sort mode for the sorted column
	EColumnSortMode::Type SortMode;

	// The path to the cache directory
	FString DirectLinkCacheDirectory;

	FOnCacheDirectoryChanged OnCacheDirectoryChanged;
	FOnCacheDirectoryReset OnCacheDirectoryReset;

	TSharedPtr<class SButton> ShowAdavancedSettingButton;
	TSharedPtr<class SImage> ShowAdavancedSettingImage;

	bool bShowingAdavancedSetting;
};
