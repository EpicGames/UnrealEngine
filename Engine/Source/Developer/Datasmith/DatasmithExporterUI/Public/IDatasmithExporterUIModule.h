// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class IDirectLinkUI;

/**
 * Static interface for the datasmith exporter UI module
 */
class IDatasmithExporterUIModule
{
public:
	/**
	 * This will return nullptr if the datasmith exporter manager initialization wasn't done with the datasmith ui argument at true
	 */
	static DATASMITHEXPORTERUI_API IDatasmithExporterUIModule* Get();

	/**
	 * Return the Direct Link UI interface
	 * @return A valid pointer if the exporter was initialized with the messaging argument being true
	 */
	virtual IDirectLinkUI* GetDirectLinkExporterUI() const = 0;

	virtual ~IDatasmithExporterUIModule() = default;
};
