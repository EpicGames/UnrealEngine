// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class IDirectLinkUI
{
public:
	/**
	 * Open the direct link connections window or move it up front if it's already open
	 */
	virtual void SetStreamWindowCenter(int InCenterX, int InCenterY) = 0;
	virtual void OpenDirectLinkStreamWindow() = 0;

	/**
	 * Get the user defined cache directory for direct link
	 */
	virtual const TCHAR* GetDirectLinkCacheDirectory() = 0;

	virtual ~IDirectLinkUI() = default;
};