// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	[SupportedPlatforms("Win64", "Mac")]
	public class DatasmithFacade : ModuleRules
	{
		public DatasmithFacade(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"DatasmithCore",
					"DirectLink",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DatasmithExporter",
					"Imath",
				}
			);
		}
	}
}
