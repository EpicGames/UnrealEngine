// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeActor.h"

#include "DatasmithFacadeActorCamera.h"
#include "DatasmithFacadeActorLight.h"
#include "DatasmithFacadeActorMesh.h"
#include "DatasmithFacadeScene.h"

// Imath third party library.
#include "Imath/ImathMatrixAlgo.h"


FDatasmithFacadeActor::FDatasmithFacadeActor(
	const TCHAR* InElementName
)
	: FDatasmithFacadeElement(FDatasmithSceneFactory::CreateActor(InElementName))
{
	TSharedPtr<IDatasmithActorElement> ActorElement = GetDatasmithActorElement();
	ActorElement->SetIsAComponent(false);
}

FDatasmithFacadeActor::FDatasmithFacadeActor(
	const TSharedRef<IDatasmithActorElement>& InInternalActor
)
	: FDatasmithFacadeElement(InInternalActor)
{}

void FDatasmithFacadeActor::SetWorldTransform(
	const double InWorldMatrix[16],
	bool bRowMajor
)
{
	FTransform WorldTransform = ConvertTransform(InWorldMatrix, bRowMajor);

	// Set the Datasmith actor world transform.
	TSharedPtr<IDatasmithActorElement> ActorElement = GetDatasmithActorElement();
	ActorElement->SetScale(WorldTransform.GetScale3D());
	ActorElement->SetRotation(WorldTransform.GetRotation());
	ActorElement->SetTranslation(WorldTransform.GetTranslation());
}

void FDatasmithFacadeActor::SetScale(
	double X,
	double Y,
	double Z
)
{
	GetDatasmithActorElement()->SetScale(FVector(X, Y, Z));
}

void FDatasmithFacadeActor::GetScale(
	double& OutX,
	double& OutY,
	double& OutZ
) const
{
	FVector ScaleVector(GetDatasmithActorElement()->GetScale());

	OutX = ScaleVector.X;
	OutY = ScaleVector.Y;
	OutZ = ScaleVector.Z;
}

void FDatasmithFacadeActor::SetRotation(
	double Pitch,
	double Yaw,
	double Roll
)
{
	GetDatasmithActorElement()->SetRotation(FQuat(FRotator(Pitch, Yaw, Roll)));
}

void FDatasmithFacadeActor::GetRotation(
	double& OutPitch,
	double& OutYaw,
	double& OutRoll
) const
{
	FRotator Rotator(GetDatasmithActorElement()->GetRotation().Rotator());

	OutPitch = Rotator.Pitch;
	OutYaw = Rotator.Yaw;
	OutRoll = Rotator.Roll;
}

void FDatasmithFacadeActor::SetRotation(
	double X,
	double Y,
	double Z,
	double W
)
{
	GetDatasmithActorElement()->SetRotation(FQuat(X, Y, Z, W));
}

void FDatasmithFacadeActor::GetRotation(
	double& OutX,
	double& OutY,
	double& OutZ,
	double& OutW
) const
{
	FQuat RotationQuat(GetDatasmithActorElement()->GetRotation());

	OutX = RotationQuat.X;
	OutY = RotationQuat.Y;
	OutZ = RotationQuat.Z;
	OutW = RotationQuat.W;
}

void FDatasmithFacadeActor::SetTranslation(
	double X,
	double Y,
	double Z
)
{
	GetDatasmithActorElement()->SetTranslation(FVector(X, Y, Z));
}

void FDatasmithFacadeActor::GetTranslation(
	double& OutX,
	double& OutY,
	double& OutZ
) const
{
	FVector TranslationVector(GetDatasmithActorElement()->GetTranslation());

	OutX = TranslationVector.X;
	OutY = TranslationVector.Y;
	OutZ = TranslationVector.Z;
}

void FDatasmithFacadeActor::SetLayer(
	const TCHAR* InLayerName
)
{
	GetDatasmithActorElement()->SetLayer(InLayerName);
}

const TCHAR* FDatasmithFacadeActor::GetLayer() const
{
	return GetDatasmithActorElement()->GetLayer();
}

void FDatasmithFacadeActor::AddTag(
	const TCHAR* InTag
)
{
	GetDatasmithActorElement()->AddTag(InTag);
}

void FDatasmithFacadeActor::ResetTags()
{
	GetDatasmithActorElement()->ResetTags();
}

int32 FDatasmithFacadeActor::GetTagsCount() const
{
	return GetDatasmithActorElement()->GetTagsCount();
}

const TCHAR* FDatasmithFacadeActor::GetTag(
	int32 TagIndex
) const
{
	return GetDatasmithActorElement()->GetTag(TagIndex);
}

bool FDatasmithFacadeActor::IsComponent() const
{
	return GetDatasmithActorElement()->IsAComponent();
}

void FDatasmithFacadeActor::SetIsComponent(
	bool bInIsComponent
)
{
	GetDatasmithActorElement()->SetIsAComponent(bInIsComponent);
}

void FDatasmithFacadeActor::SetVisibility(
	bool bInVisibility
)
{
	GetDatasmithActorElement()->SetVisibility(bInVisibility);
}

bool FDatasmithFacadeActor::GetVisibility() const
{
	return GetDatasmithActorElement()->GetVisibility();
}

void FDatasmithFacadeActor::SetCastShadow(bool bInCastShadow)
{
	GetDatasmithActorElement()->SetCastShadow(bInCastShadow);
}

bool FDatasmithFacadeActor::GetCastShadow() const
{
	return GetDatasmithActorElement()->GetCastShadow();
}

FDatasmithFacadeActor::EActorType FDatasmithFacadeActor::GetActorType() const
{
	return FDatasmithFacadeActor::GetActorType(GetDatasmithActorElement());
}

FDatasmithFacadeActor::EActorType FDatasmithFacadeActor::GetActorType(
	const TSharedPtr<const IDatasmithActorElement>& InActor
)
{
	if (InActor->IsA(EDatasmithElementType::DirectionalLight))
	{
		return EActorType::DirectionalLight;
	}
	else if (InActor->IsA(EDatasmithElementType::AreaLight))
	{
		return EActorType::AreaLight;
	}
	else if (InActor->IsA(EDatasmithElementType::EnvironmentLight))
	{
		return EActorType::Unsupported;
	}
	else if (InActor->IsA(EDatasmithElementType::LightmassPortal))
	{
		return EActorType::LightmassPortal;
	}
	else if (InActor->IsA(EDatasmithElementType::PointLight))
	{
		return EActorType::PointLight;
	}
	else if (InActor->IsA(EDatasmithElementType::SpotLight))
	{
		return EActorType::SpotLight;
	}
	else if (InActor->IsA(EDatasmithElementType::Light))
	{
		return EActorType::Unsupported;
	}
	else if (InActor->IsA(EDatasmithElementType::StaticMeshActor))
	{
		return EActorType::StaticMeshActor;
	}
	else if (InActor->IsA(EDatasmithElementType::Camera))
	{
		return EActorType::Camera;
	}
	else if (InActor->IsA(EDatasmithElementType::Light
		| EDatasmithElementType::HierarchicalInstanceStaticMesh
		| EDatasmithElementType::CustomActor))
	{
		return EActorType::Unsupported;
	}
	else if (InActor->IsA(EDatasmithElementType::Actor))
	{
		return EActorType::Actor;
	}

	return EActorType::Unsupported;
}

FDatasmithFacadeActor* FDatasmithFacadeActor::GetNewFacadeActorFromSharedPtr(
	const TSharedPtr<IDatasmithActorElement>& InActor
)
{
	if (InActor)
	{
		EActorType ActorType = GetActorType(InActor);
		TSharedRef<IDatasmithActorElement> ActorRef = InActor.ToSharedRef();

		switch (ActorType)
		{
		case EActorType::DirectionalLight:
			return new FDatasmithFacadeDirectionalLight(StaticCastSharedRef<IDatasmithDirectionalLightElement>(ActorRef));
		case EActorType::AreaLight:
			return new FDatasmithFacadeAreaLight(StaticCastSharedRef<IDatasmithAreaLightElement>(ActorRef));
		case EActorType::LightmassPortal:
			return new FDatasmithFacadeLightmassPortal(StaticCastSharedRef<IDatasmithLightmassPortalElement>(ActorRef));
		case EActorType::PointLight:
			return new FDatasmithFacadePointLight(StaticCastSharedRef<IDatasmithPointLightElement>(ActorRef));
		case EActorType::SpotLight:
			return new FDatasmithFacadeSpotLight(StaticCastSharedRef<IDatasmithSpotLightElement>(ActorRef));
		case EActorType::StaticMeshActor:
			return new FDatasmithFacadeActorMesh(StaticCastSharedRef<IDatasmithMeshActorElement>(ActorRef));
		case EActorType::Camera:
			return new FDatasmithFacadeActorCamera(StaticCastSharedRef<IDatasmithCameraActorElement>(ActorRef));
		case EActorType::Actor:
			return new FDatasmithFacadeActor(ActorRef);
		case EActorType::Unsupported:
		default:
			return nullptr;
		}
	}

	return nullptr;
}

void FDatasmithFacadeActor::AddChild(
	FDatasmithFacadeActor* InChildActorPtr
)
{
	if (InChildActorPtr != nullptr)
	{
		GetDatasmithActorElement()->AddChild(InChildActorPtr->GetDatasmithActorElement());
	}
}

int32 FDatasmithFacadeActor::GetChildrenCount() const
{
	return GetDatasmithActorElement()->GetChildrenCount();
}

FDatasmithFacadeActor* FDatasmithFacadeActor::GetNewChild(
	int32 InIndex
)
{
	TSharedPtr<IDatasmithActorElement> ChildActor = GetDatasmithActorElement()->GetChild(InIndex);

	return GetNewFacadeActorFromSharedPtr(ChildActor);
}

void FDatasmithFacadeActor::RemoveChild(
	FDatasmithFacadeActor* InChild
)
{
	if (InChild)
	{
		GetDatasmithActorElement()->RemoveChild(InChild->GetDatasmithActorElement());
	}
}

FTransform FDatasmithFacadeActor::ConvertTransform(
	const double InSourceMatrix[16],
	bool bRowMajor
) const
{
	// We use Imath::extractAndRemoveScalingAndShear() because FMatrix::ExtractScaling() is deemed unreliable.

	// Set up a scaling and rotation matrix.
	Imath::Matrix44<double> Matrix;

	if (bRowMajor)
	{
		Matrix = Imath::Matrix44<double>(InSourceMatrix[0], InSourceMatrix[4], InSourceMatrix[8],  0.0,
										InSourceMatrix[1], InSourceMatrix[5], InSourceMatrix[9],  0.0,
										InSourceMatrix[2], InSourceMatrix[6], InSourceMatrix[10], 0.0,
										0.0			   , 0.0            , 0.0				, 1.0);
	}
	else
	{
		Matrix = Imath::Matrix44<double>(InSourceMatrix[0], InSourceMatrix[1], InSourceMatrix[2],  0.0,
										InSourceMatrix[4], InSourceMatrix[5], InSourceMatrix[6],  0.0,
										InSourceMatrix[8], InSourceMatrix[9], InSourceMatrix[10], 0.0,
										0.0			  , 0.0              , 0.0               , 1.0);
	}

	// Remove any scaling from the matrix and get the scale vector that was initially present.
	Imath::Vec3<double> Scale;
	Imath::Vec3<double> Shear;
	bool bExtracted = Imath::extractAndRemoveScalingAndShear<double>(Matrix, Scale, Shear, false);

	if (!bExtracted)
	{
		// TODO: Append a message to the build summary.
		FString Msg = FString::Printf(TEXT("WARNING: Actor %ls (%ls) has some zero scaling"), GetName(), GetLabel());

		return FTransform::Identity;
	}

	FVector TransformScale3D;

	// Initialize a rotation quaternion with the rotation matrix.
	Imath::Quat<double> Quaternion = Imath::extractQuat<double>(Matrix);

	switch (WorldCoordinateSystemType)
	{
		case ECoordinateSystemType::LeftHandedYup:
		{
			// Set the Datasmith actor scale.
			TransformScale3D = FVector(Scale.x, Scale.z, Scale.y);

			// Convert the left-handed Y-up coordinate rotation into an Unreal left-handed Z-up coordinate rotation.
			// This is done by doing a 90 degree rotation about the X axis.
			float Y = Quaternion.v.y;
			float Z = Quaternion.v.z;
			Quaternion.v.y = -Z;
			Quaternion.v.z =  Y;
			Quaternion.normalize();

			break;
		}

		case ECoordinateSystemType::LeftHandedZup:
		{
			// Set the Datasmith actor scale.
			TransformScale3D = FVector(Scale.x, Scale.y, Scale.z);

			break;
		}

		case ECoordinateSystemType::RightHandedZup:
		{
			// Set the Datasmith actor scale.
			TransformScale3D = FVector(Scale.x, Scale.y, Scale.z);

			// Convert the right-handed Z-up coordinate rotation into an Unreal left-handed Z-up coordinate rotation.
			// This is done by inverting the X and Z components of the quaternion to mirror the quaternion on the XZ-plane.
			Quaternion.v.x = -Quaternion.v.x;
			Quaternion.v.z = -Quaternion.v.z;
			Quaternion.normalize();

			break;
		}
	}

	// Make sure Unreal will be able to handle the rotation quaternion.
	double              Angle = Quaternion.angle();
	Imath::Vec3<double> Axis  = Quaternion.axis();
	FQuat TransformRotation = FQuat(FVector(Axis.x, Axis.y, Axis.z), Angle);

	// Scale and convert the source translation into a Datasmith actor translation.
	FVector TransformTranslation = bRowMajor
		? ConvertPosition(InSourceMatrix[4], InSourceMatrix[7], InSourceMatrix[11])
		: ConvertPosition(InSourceMatrix[12], InSourceMatrix[13], InSourceMatrix[14]);

	return FTransform(TransformRotation, TransformTranslation, TransformScale3D);
}

TSharedRef<IDatasmithActorElement> FDatasmithFacadeActor::GetDatasmithActorElement() const
{
	return StaticCastSharedRef<IDatasmithActorElement>(InternalDatasmithElement);
}
