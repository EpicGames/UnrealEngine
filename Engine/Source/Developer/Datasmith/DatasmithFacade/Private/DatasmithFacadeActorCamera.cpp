// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeActorCamera.h"
#include "IDatasmithSceneElements.h"
#include "Math/RotationMatrix.h"

FDatasmithFacadeActorCamera::FDatasmithFacadeActorCamera(
	const TCHAR* InElementName
)
	: FDatasmithFacadeActor(FDatasmithSceneFactory::CreateCameraActor(InElementName))
{
	SetCameraPosition(0.0, 0.0, 0.0); // default camera position at world origin
	SetSensorWidth(36.0);                  // default Datasmith sensor width of 36 mm
	SetAspectRatio(16.0f / 9.0f);          // default Datasmith aspect ratio of 16:9
	SetFocusDistance(1000.0);              // default Datasmith focus distance of 1000 centimeters
	SetFocalLength(35.0);                  // default Datasmith focal length of 35 millimeters
	SetLookAtAllowRoll(false);
	GetDatasmithActorCameraElement()->SetEnableDepthOfField(false); // no depth of field
	GetDatasmithActorCameraElement()->SetFStop(5.6f); // default Datasmith f-stop of f/5.6
}

FDatasmithFacadeActorCamera::FDatasmithFacadeActorCamera(
	const TSharedRef<IDatasmithCameraActorElement>& InInternalElement
)
	: FDatasmithFacadeActor(InInternalElement)
{}

void FDatasmithFacadeActorCamera::SetCameraPosition(
	double InX,
	double InY,
	double InZ
)
{
	// Scale and convert the camera position into a Datasmith actor world translation.
	GetDatasmithActorCameraElement()->SetTranslation(ConvertPosition(InX, InY, InZ));
}

void FDatasmithFacadeActorCamera::SetCameraRotation(
	double InForwardX,
	double InForwardY,
	double InForwardZ,
	double InUpX,
	double InUpY,
	double InUpZ
)
{
	// Convert the camera orientation into a Datasmith actor world look-at rotation quaternion.
	FVector XAxis = ConvertDirection(InForwardX, InForwardY, InForwardZ);
	FVector ZAxis = ConvertDirection(InUpX, InUpY, InUpZ);
	GetDatasmithActorCameraElement()->SetRotation(FQuat(FRotationMatrix::MakeFromXZ(XAxis, ZAxis))); // axis vectors do not need to be normalized
}

void FDatasmithFacadeActorCamera::SetSensorWidth(
	float InSensorWidth
)
{
	GetDatasmithActorCameraElement()->SetSensorWidth(InSensorWidth);
}

float FDatasmithFacadeActorCamera::GetSensorWidth() const
{
	return GetDatasmithActorCameraElement()->GetSensorWidth();
}

void FDatasmithFacadeActorCamera::SetAspectRatio(
	float InAspectRatio
)
{
	GetDatasmithActorCameraElement()->SetSensorAspectRatio(InAspectRatio);
}

float FDatasmithFacadeActorCamera::GetAspectRatio() const
{
	return GetDatasmithActorCameraElement()->GetSensorAspectRatio();
}

void FDatasmithFacadeActorCamera::SetFocusDistance(
	float InTargetX,
	float InTargetY,
	float InTargetZ
)
{
	TSharedRef<IDatasmithCameraActorElement> CameraActor(GetDatasmithActorCameraElement());
	FVector CameraPosition(CameraActor->GetTranslation());
	FVector Target(InTargetX, InTargetY, InTargetZ);

	FVector DistanceVector = Target - CameraPosition;
	CameraActor->SetFocusDistance((float)DistanceVector.Size() * WorldUnitScale);
}

void FDatasmithFacadeActorCamera::SetFocusDistance(
	float InFocusDistance
)
{
	GetDatasmithActorCameraElement()->SetFocusDistance(InFocusDistance * WorldUnitScale);
}

float FDatasmithFacadeActorCamera::GetFocusDistance() const
{
	return GetDatasmithActorCameraElement()->GetFocusDistance() / WorldUnitScale;
};

void FDatasmithFacadeActorCamera::SetFocalLength(
	float InFocalLength
)
{
	GetDatasmithActorCameraElement()->SetFocalLength(InFocalLength);
}

void FDatasmithFacadeActorCamera::SetFocalLength(
	float InFOV,
	bool  bInVerticalFOV
)
{
	const float SensorWidth = GetDatasmithActorCameraElement()->GetSensorWidth();
	const float AspectRatio = GetDatasmithActorCameraElement()->GetSensorAspectRatio();
	const float FocalLength = float(( bInVerticalFOV ? ( SensorWidth / AspectRatio ) : SensorWidth ) / ( 2.0 * tan(FMath::DegreesToRadians<double>(InFOV) / 2.0) ));

	SetFocalLength(FocalLength);
}

float FDatasmithFacadeActorCamera::GetFocalLength() const
{
	return GetDatasmithActorCameraElement()->GetFocalLength();
}

void FDatasmithFacadeActorCamera::SetLookAtActor(
	const TCHAR* InActorName
)
{
	GetDatasmithActorCameraElement()->SetLookAtActor(InActorName);
}

const TCHAR* FDatasmithFacadeActorCamera::GetLookAtActor() const
{
	return GetDatasmithActorCameraElement()->GetLookAtActor();
}

void FDatasmithFacadeActorCamera::SetLookAtAllowRoll(
	bool bInAllow
)
{
	GetDatasmithActorCameraElement()->SetLookAtAllowRoll(bInAllow);
}

bool FDatasmithFacadeActorCamera::GetLookAtAllowRoll() const
{
	return GetDatasmithActorCameraElement()->GetLookAtAllowRoll();
}

void FDatasmithFacadeActorCamera::SetEnableDepthOfField(
	bool bInEnableDepthOfField
)
{
	GetDatasmithActorCameraElement()->SetEnableDepthOfField(bInEnableDepthOfField);
}

bool FDatasmithFacadeActorCamera::GetEnableDepthOfField() const
{
	return GetDatasmithActorCameraElement()->GetEnableDepthOfField();
}

float FDatasmithFacadeActorCamera::GetFStop() const
{
	return GetDatasmithActorCameraElement()->GetFStop();
}

void FDatasmithFacadeActorCamera::SetFStop(
	float InFStop
)
{
	GetDatasmithActorCameraElement()->SetFStop(InFStop);
}

TSharedRef<IDatasmithCameraActorElement> FDatasmithFacadeActorCamera::GetDatasmithActorCameraElement() const
{
	return StaticCastSharedRef<IDatasmithCameraActorElement>(InternalDatasmithElement);
}