// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeActorLight.h"

// Datasmith SDK.
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

FDatasmithFacadeActorLight::FDatasmithFacadeActorLight(
	const TSharedRef<IDatasmithLightActorElement>& InInternalActor
)
	: FDatasmithFacadeActor(InInternalActor)
{}

bool FDatasmithFacadeActorLight::IsEnabled() const
{
	return GetDatasmithLightActorElement()->IsEnabled();
}

void FDatasmithFacadeActorLight::SetEnabled(
	bool bIsEnabled
)
{
	GetDatasmithLightActorElement()->SetEnabled(bIsEnabled);
}

double FDatasmithFacadeActorLight::GetIntensity() const
{
	return GetDatasmithLightActorElement()->GetIntensity();
}

void FDatasmithFacadeActorLight::SetIntensity(
	double Intensity
)
{
	return GetDatasmithLightActorElement()->SetIntensity(Intensity);
}

void FDatasmithFacadeActorLight::GetColor(
	uint8& OutR,
	uint8& OutG,
	uint8& OutB,
	uint8& OutA
) const
{
	FColor Color(GetDatasmithLightActorElement()->GetColor().ToFColor( /*bSRGB=*/true));
	OutR = Color.R;
	OutG = Color.G;
	OutB = Color.B;
	OutA = Color.A;
}

void FDatasmithFacadeActorLight::GetColor(
	float& OutR,
	float& OutG,
	float& OutB,
	float& OutA
) const
{
	FLinearColor Color(GetDatasmithLightActorElement()->GetColor());
	OutR = Color.R;
	OutG = Color.G;
	OutB = Color.B;
	OutA = Color.A;
}

void FDatasmithFacadeActorLight::SetColor(
	uint8 InR,
	uint8 InG,
	uint8 InB,
	uint8 InA
)
{
	GetDatasmithLightActorElement()->SetColor(FLinearColor(FColor(InR, InG, InB, InA)));
}

void FDatasmithFacadeActorLight::SetColor(
	float InR,
	float InG,
	float InB,
	float InA
)
{
	GetDatasmithLightActorElement()->SetColor(FLinearColor(InR, InG, InB, InA));
}

double FDatasmithFacadeActorLight::GetTemperature() const
{
	return GetDatasmithLightActorElement()->GetTemperature();
}

void FDatasmithFacadeActorLight::SetTemperature(
	double Temperature
)
{
	GetDatasmithLightActorElement()->SetTemperature(Temperature);
}

bool FDatasmithFacadeActorLight::GetUseTemperature() const
{
	return GetDatasmithLightActorElement()->GetUseTemperature();
}

void FDatasmithFacadeActorLight::SetUseTemperature(
	bool bUseTemperature
)
{
	GetDatasmithLightActorElement()->SetUseTemperature(bUseTemperature);
}

const TCHAR* FDatasmithFacadeActorLight::GetIesFile() const
{
	return GetDatasmithLightActorElement()->GetIesFile();
}

void FDatasmithFacadeActorLight::WriteIESFile(
	const TCHAR* InIESFileFolder,
	const TCHAR* InIESFileName,
	const TCHAR* InIESData
)
{
	FString IESFilePath = FPaths::Combine(FString(InIESFileFolder), FString(InIESFileName));

	// Set the file path of the IES definition file.
	SetIesFile(*IESFilePath);

	// Write the IES definition data in the IES definition file.
	FFileHelper::SaveStringToFile(FString(InIESData), *IESFilePath);
}

void FDatasmithFacadeActorLight::SetIesFile(
	const TCHAR* IesFile
)
{
	GetDatasmithLightActorElement()->SetIesFile(IesFile);
}

bool FDatasmithFacadeActorLight::GetUseIes() const
{
	return GetDatasmithLightActorElement()->GetUseIes();
}

void FDatasmithFacadeActorLight::SetUseIes(
	bool bUseIes
)
{
	GetDatasmithLightActorElement()->SetUseIes(bUseIes);
}

double FDatasmithFacadeActorLight::GetIesBrightnessScale() const
{
	return GetDatasmithLightActorElement()->GetIesBrightnessScale();
}

void FDatasmithFacadeActorLight::SetIesBrightnessScale(
	double IesBrightnessScale
)
{
	GetDatasmithLightActorElement()->SetIesBrightnessScale(IesBrightnessScale);
}

bool FDatasmithFacadeActorLight::GetUseIesBrightness() const
{
	return GetDatasmithLightActorElement()->GetUseIesBrightness();
}

void FDatasmithFacadeActorLight::SetUseIesBrightness(
	bool bUseIesBrightness
)
{
	GetDatasmithLightActorElement()->SetUseIesBrightness(bUseIesBrightness);
}

void FDatasmithFacadeActorLight::GetIesRotation(
	float& OutX,
	float& OutY,
	float& OutZ,
	float& OutW
) const
{
	FQuat IesQuat(GetDatasmithLightActorElement()->GetIesRotation());
	OutX = (float)IesQuat.X;
	OutY = (float)IesQuat.Y;
	OutZ = (float)IesQuat.Z;
	OutW = (float)IesQuat.W;
}

void FDatasmithFacadeActorLight::GetIesRotation(
	float& OutPitch,
	float& OutYaw,
	float& OutRoll
) const
{
	FRotator IesRotator(GetDatasmithLightActorElement()->GetIesRotation().Rotator());
	OutPitch = (float)IesRotator.Pitch;
	OutYaw = (float)IesRotator.Yaw;
	OutRoll = (float)IesRotator.Roll;
}

void FDatasmithFacadeActorLight::SetIesRotation(
	float X,
	float Y,
	float Z,
	float W
)
{
	GetDatasmithLightActorElement()->SetIesRotation(FQuat(X, Y, Z, W));
}

void FDatasmithFacadeActorLight::SetIesRotation(
	float Pitch,
	float Yaw,
	float Roll
)
{
	GetDatasmithLightActorElement()->SetIesRotation(FQuat(FRotator(Pitch, Yaw, Roll)));
}

FDatasmithFacadeMaterialID* FDatasmithFacadeActorLight::GetNewLightFunctionMaterial()
{
	if (TSharedPtr<IDatasmithMaterialIDElement> MaterialID = GetDatasmithLightActorElement()->GetLightFunctionMaterial())
	{
		return new FDatasmithFacadeMaterialID(MaterialID.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeActorLight::SetLightFunctionMaterial(
	FDatasmithFacadeMaterialID& InMaterial
)
{
	GetDatasmithLightActorElement()->SetLightFunctionMaterial(InMaterial.GetMaterialIDElement());
}

void FDatasmithFacadeActorLight::SetLightFunctionMaterial(
	const TCHAR* InMaterialName
)
{
	GetDatasmithLightActorElement()->SetLightFunctionMaterial(InMaterialName);
}

TSharedRef<IDatasmithLightActorElement> FDatasmithFacadeActorLight::GetDatasmithLightActorElement() const
{
	return StaticCastSharedRef<IDatasmithLightActorElement>(InternalDatasmithElement);
}

FDatasmithFacadePointLight::FDatasmithFacadePointLight(
	const TCHAR* InElementName
)
	: FDatasmithFacadeActorLight(FDatasmithSceneFactory::CreatePointLight(InElementName))
{}

FDatasmithFacadePointLight::FDatasmithFacadePointLight(
	const TSharedRef<IDatasmithPointLightElement>& InInternalActor
)
	: FDatasmithFacadeActorLight(InInternalActor)
{}

void FDatasmithFacadePointLight::SetIntensityUnits(
	EPointLightIntensityUnit InUnits
)
{
	GetDatasmithPointLightElement()->SetIntensityUnits(static_cast<EDatasmithLightUnits>( InUnits ));
}

FDatasmithFacadePointLight::EPointLightIntensityUnit FDatasmithFacadePointLight::GetIntensityUnits() const
{
	return static_cast<FDatasmithFacadePointLight::EPointLightIntensityUnit>( GetDatasmithPointLightElement()->GetIntensityUnits() );;
}

float FDatasmithFacadePointLight::GetSourceRadius() const
{
	return GetDatasmithPointLightElement()->GetSourceRadius() / WorldUnitScale;
}

void FDatasmithFacadePointLight::SetSourceRadius(
	float SourceRadius
)
{
	GetDatasmithPointLightElement()->SetSourceRadius(SourceRadius * WorldUnitScale);
}

float FDatasmithFacadePointLight::GetSourceLength() const
{
	return GetDatasmithPointLightElement()->GetSourceLength() / WorldUnitScale;
}

void FDatasmithFacadePointLight::SetSourceLength(
	float SourceLength
)
{
	GetDatasmithPointLightElement()->SetSourceLength(SourceLength * WorldUnitScale);
}

float FDatasmithFacadePointLight::GetAttenuationRadius() const
{
	return GetDatasmithPointLightElement()->GetAttenuationRadius() / WorldUnitScale;
}
void FDatasmithFacadePointLight::SetAttenuationRadius(
	float AttenuationRadius
)
{
	GetDatasmithPointLightElement()->SetAttenuationRadius(AttenuationRadius * WorldUnitScale);
}

TSharedRef<IDatasmithPointLightElement> FDatasmithFacadePointLight::GetDatasmithPointLightElement() const
{
	return StaticCastSharedRef<IDatasmithPointLightElement>(InternalDatasmithElement);
}

FDatasmithFacadeSpotLight::FDatasmithFacadeSpotLight(
	const TCHAR* InElementName
)
	: FDatasmithFacadePointLight(FDatasmithSceneFactory::CreateSpotLight(InElementName))
{}

FDatasmithFacadeSpotLight::FDatasmithFacadeSpotLight(
	const TSharedRef<IDatasmithSpotLightElement>& InInternalActor
)
	: FDatasmithFacadePointLight(InInternalActor)
{}

float FDatasmithFacadeSpotLight::GetInnerConeAngle() const
{
	return GetDatasmithSpotLightElement()->GetInnerConeAngle();
}

void FDatasmithFacadeSpotLight::SetInnerConeAngle(
	float InnerConeAngle
)
{
	GetDatasmithSpotLightElement()->SetInnerConeAngle(InnerConeAngle);
}

float FDatasmithFacadeSpotLight::GetOuterConeAngle() const
{
	return GetDatasmithSpotLightElement()->GetOuterConeAngle();
}

void FDatasmithFacadeSpotLight::SetOuterConeAngle(
	float OuterConeAngle
)
{
	GetDatasmithSpotLightElement()->SetOuterConeAngle(OuterConeAngle);
}

TSharedRef<IDatasmithSpotLightElement> FDatasmithFacadeSpotLight::GetDatasmithSpotLightElement() const
{
	return StaticCastSharedRef<IDatasmithSpotLightElement>(InternalDatasmithElement);
}

FDatasmithFacadeDirectionalLight::FDatasmithFacadeDirectionalLight(
	const TCHAR* InElementName
)
	: FDatasmithFacadeActorLight(FDatasmithSceneFactory::CreateDirectionalLight(InElementName))
{}

FDatasmithFacadeDirectionalLight::FDatasmithFacadeDirectionalLight(
	const TSharedRef<IDatasmithDirectionalLightElement>& InActorElement
)
	: FDatasmithFacadeActorLight(InActorElement)
{}

FDatasmithFacadeAreaLight::FDatasmithFacadeAreaLight(
	const TCHAR* InElementName
)
	: FDatasmithFacadeSpotLight(FDatasmithSceneFactory::CreateAreaLight(InElementName))
{}

FDatasmithFacadeAreaLight::FDatasmithFacadeAreaLight(
	const TSharedRef<IDatasmithAreaLightElement>& InActorElement
)
	: FDatasmithFacadeSpotLight(InActorElement)
{}

FDatasmithFacadeAreaLight::EAreaLightShape FDatasmithFacadeAreaLight::GetLightShape() const
{
	return static_cast<EAreaLightShape>( GetDatasmithAreaLightElement()->GetLightShape() );
}

void FDatasmithFacadeAreaLight::SetLightShape(
	EAreaLightShape Shape
)
{
	GetDatasmithAreaLightElement()->SetLightShape(static_cast<EDatasmithLightShape> ( Shape ));
}

void FDatasmithFacadeAreaLight::SetLightType(
	EAreaLightType LightType
)
{
	GetDatasmithAreaLightElement()->SetLightType(static_cast<EDatasmithAreaLightType>( LightType ));
}

FDatasmithFacadeAreaLight::EAreaLightType FDatasmithFacadeAreaLight::GetLightType() const
{
	return static_cast<EAreaLightType>( GetDatasmithAreaLightElement()->GetLightType() );
}

void FDatasmithFacadeAreaLight::SetWidth(
	float InWidth
)
{
	GetDatasmithAreaLightElement()->SetWidth(InWidth * WorldUnitScale);
}

float FDatasmithFacadeAreaLight::GetWidth() const
{
	return GetDatasmithAreaLightElement()->GetWidth() / WorldUnitScale;
}

void FDatasmithFacadeAreaLight::SetLength(
	float InLength
)
{
	GetDatasmithAreaLightElement()->SetLength(InLength * WorldUnitScale);
}

float FDatasmithFacadeAreaLight::GetLength() const
{
	return GetDatasmithAreaLightElement()->GetLength() / WorldUnitScale;
}

TSharedRef<IDatasmithAreaLightElement> FDatasmithFacadeAreaLight::GetDatasmithAreaLightElement() const
{
	return StaticCastSharedRef<IDatasmithAreaLightElement>(InternalDatasmithElement);
}

FDatasmithFacadeLightmassPortal::FDatasmithFacadeLightmassPortal(
	const TCHAR* InElementName
)
	: FDatasmithFacadePointLight(FDatasmithSceneFactory::CreateLightmassPortal(InElementName))
{}

FDatasmithFacadeLightmassPortal::FDatasmithFacadeLightmassPortal(
	const TSharedRef<IDatasmithLightmassPortalElement>& InActorElement
)
	: FDatasmithFacadePointLight(InActorElement)
{}
