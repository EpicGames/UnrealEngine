// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeActorMesh.h"

#include "DatasmithFacadeMaterialID.h"
#include "DatasmithFacadeScene.h"


FDatasmithFacadeActorMesh::FDatasmithFacadeActorMesh(
	const TCHAR* InElementName
)
	: FDatasmithFacadeActor(FDatasmithSceneFactory::CreateMeshActor(InElementName))
{}

FDatasmithFacadeActorMesh::FDatasmithFacadeActorMesh(
	const TSharedRef<IDatasmithMeshActorElement>& InInternalElement
)
	: FDatasmithFacadeActor(InInternalElement)
{}

void FDatasmithFacadeActorMesh::SetMesh(
	const TCHAR* InMeshName
)
{
	GetDatasmithMeshActorElement()->SetStaticMeshPathName(InMeshName);
}

const TCHAR* FDatasmithFacadeActorMesh::GetMeshName() const
{
	return GetDatasmithMeshActorElement()->GetStaticMeshPathName();
}

void FDatasmithFacadeActorMesh::AddMaterialOverride(
	const TCHAR* MaterialName,
	int32 Id
)
{
	GetDatasmithMeshActorElement()->AddMaterialOverride(MaterialName, Id);
}

void FDatasmithFacadeActorMesh::AddMaterialOverride(
	FDatasmithFacadeMaterialID& Material
)
{
	GetDatasmithMeshActorElement()->AddMaterialOverride(Material.GetMaterialIDElement());
}

int32 FDatasmithFacadeActorMesh::GetMaterialOverridesCount() const
{
	return GetDatasmithMeshActorElement()->GetMaterialOverridesCount();
}

FDatasmithFacadeMaterialID* FDatasmithFacadeActorMesh::GetNewMaterialOverride(
	int32 MaterialOverrideIndex
)
{
	if (TSharedPtr<IDatasmithMaterialIDElement> MaterialID = GetDatasmithMeshActorElement()->GetMaterialOverride(MaterialOverrideIndex))
	{
		return new FDatasmithFacadeMaterialID(MaterialID.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeActorMesh::RemoveMaterialOverride(
	FDatasmithFacadeMaterialID& Material
)
{
	GetDatasmithMeshActorElement()->RemoveMaterialOverride(Material.GetMaterialIDElement());
}

void FDatasmithFacadeActorMesh::ResetMaterialOverrides()
{
	GetDatasmithMeshActorElement()->ResetMaterialOverrides();
}

TSharedRef<IDatasmithMeshActorElement> FDatasmithFacadeActorMesh::GetDatasmithMeshActorElement() const
{
	return StaticCastSharedRef<IDatasmithMeshActorElement>(InternalDatasmithElement);
}