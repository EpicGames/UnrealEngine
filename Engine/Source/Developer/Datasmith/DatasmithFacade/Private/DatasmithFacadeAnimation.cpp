// Copyright Epic Games, Inc. All Rights Reserved.


#include "DatasmithFacadeAnimation.h"
#include "DatasmithFacadeScene.h"

// FDatasmithFacadeBaseAnimation

FDatasmithFacadeBaseAnimation::FDatasmithFacadeBaseAnimation(const TSharedRef<IDatasmithBaseAnimationElement>& InInternalAnimation)
	: FDatasmithFacadeElement( InInternalAnimation )
{
}

bool FDatasmithFacadeBaseAnimation::IsSubType( const EDatasmithFacadeElementAnimationSubType AnimSubType ) const
{
	return GetDatasmithBaseAnimation()->IsSubType(static_cast<EDatasmithElementAnimationSubType>(AnimSubType));
}

void FDatasmithFacadeBaseAnimation::SetCompletionMode(EDatasmithFacadeCompletionMode CompletionMode)
{
	GetDatasmithBaseAnimation()->SetCompletionMode(static_cast<EDatasmithCompletionMode>(CompletionMode));
}

EDatasmithFacadeCompletionMode FDatasmithFacadeBaseAnimation::GetCompletionMode() const
{
	return static_cast<EDatasmithFacadeCompletionMode>(GetDatasmithBaseAnimation()->GetCompletionMode());
}

TSharedRef<IDatasmithBaseAnimationElement> FDatasmithFacadeBaseAnimation::GetDatasmithBaseAnimation() const
{
	return StaticCastSharedRef<IDatasmithBaseAnimationElement>( InternalDatasmithElement );
}

// FDatasmithFacadeTransformAnimation

FDatasmithFacadeTransformAnimation::FDatasmithFacadeTransformAnimation(const TCHAR* InName)
	: FDatasmithFacadeBaseAnimation( FDatasmithSceneFactory::CreateTransformAnimation(InName) )
{
}

FDatasmithFacadeTransformAnimation::FDatasmithFacadeTransformAnimation(
	const TSharedRef<IDatasmithTransformAnimationElement>& InInternalAnimation
) : FDatasmithFacadeBaseAnimation( InInternalAnimation )
{
}

TSharedRef<IDatasmithTransformAnimationElement> FDatasmithFacadeTransformAnimation::GetDatasmithTransformAnimation() const
{
	return StaticCastSharedRef<IDatasmithTransformAnimationElement>( InternalDatasmithElement );
}

void FDatasmithFacadeTransformAnimation::AddFrame(EDatasmithFacadeAnimationTransformType TransformType, int32 FrameNumber, float X, float Y, float Z)
{
	GetDatasmithTransformAnimation()->AddFrame(
		static_cast<EDatasmithTransformType>(TransformType),
		FDatasmithTransformFrameInfo(FrameNumber, X, Y, Z)
	);
}

int32 FDatasmithFacadeTransformAnimation::GetFramesCount(EDatasmithFacadeAnimationTransformType TransformType) const
{
	return GetDatasmithTransformAnimation()->GetFramesCount(static_cast<EDatasmithTransformType>(TransformType));
}

void FDatasmithFacadeTransformAnimation::SetCurveInterpMode(EDatasmithFacadeAnimationTransformType TransformType, EDatasmithFacadeCurveInterpMode CurveInterpMode)
{
	GetDatasmithTransformAnimation()->SetCurveInterpMode(
		static_cast<EDatasmithTransformType>(TransformType),
		static_cast<EDatasmithCurveInterpMode>(CurveInterpMode)
	);
}

EDatasmithFacadeCurveInterpMode FDatasmithFacadeTransformAnimation::GetCurveInterpMode(EDatasmithFacadeAnimationTransformType TransformType) const
{
	return static_cast<EDatasmithFacadeCurveInterpMode>(GetDatasmithTransformAnimation()->GetCurveInterpMode(static_cast<EDatasmithTransformType>(TransformType)));
}

void FDatasmithFacadeTransformAnimation::GetFrame(EDatasmithFacadeAnimationTransformType TransformType, int32 Index, int32& OutFrameNumber, double& OutX, double& OutY, double& OutZ) const
{
	const FDatasmithTransformFrameInfo& FrameInfo = GetDatasmithTransformAnimation()->GetFrame(static_cast<EDatasmithTransformType>(TransformType), Index);
	OutFrameNumber = FrameInfo.FrameNumber;
	OutX = FrameInfo.X;
	OutY = FrameInfo.Y;
	OutZ = FrameInfo.Z;
}

void FDatasmithFacadeTransformAnimation::RemoveFrame(EDatasmithFacadeAnimationTransformType TransformType, int32 Index)
{
	GetDatasmithTransformAnimation()->RemoveFrame(static_cast<EDatasmithTransformType>(TransformType), Index);
}

EDatasmithFacadeTransformChannels FDatasmithFacadeTransformAnimation::GetEnabledTransformChannels() const
{
	return static_cast<EDatasmithFacadeTransformChannels>(GetDatasmithTransformAnimation()->GetEnabledTransformChannels());
}

void FDatasmithFacadeTransformAnimation::SetEnabledTransformChannels(EDatasmithFacadeTransformChannels Channels)
{
	GetDatasmithTransformAnimation()->SetEnabledTransformChannels(static_cast<EDatasmithTransformChannels>(Channels));
}

// FDatasmithFacadeVisibilityAnimation

FDatasmithFacadeVisibilityAnimation::FDatasmithFacadeVisibilityAnimation(const TCHAR* InName)
	: FDatasmithFacadeBaseAnimation( FDatasmithSceneFactory::CreateVisibilityAnimation(InName) )
{
}

FDatasmithFacadeVisibilityAnimation::FDatasmithFacadeVisibilityAnimation(
	const TSharedRef<IDatasmithVisibilityAnimationElement>& InInternalAnimation
) : FDatasmithFacadeBaseAnimation( InInternalAnimation )
{
}

TSharedRef<IDatasmithVisibilityAnimationElement> FDatasmithFacadeVisibilityAnimation::GetDatasmithVisibilityAnimation() const
{
	return StaticCastSharedRef<IDatasmithVisibilityAnimationElement>( InternalDatasmithElement );
}

void FDatasmithFacadeVisibilityAnimation::AddFrame(int32 FrameNumber, bool bVisible)
{
	GetDatasmithVisibilityAnimation()->AddFrame(FDatasmithVisibilityFrameInfo(FrameNumber, bVisible));
}

int32 FDatasmithFacadeVisibilityAnimation::GetFramesCount() const
{
	return GetDatasmithVisibilityAnimation()->GetFramesCount();
}

void FDatasmithFacadeVisibilityAnimation::SetCurveInterpMode(EDatasmithFacadeCurveInterpMode CurveInterpMode)
{
	GetDatasmithVisibilityAnimation()->SetCurveInterpMode(static_cast<EDatasmithCurveInterpMode>(CurveInterpMode));
}

EDatasmithFacadeCurveInterpMode FDatasmithFacadeVisibilityAnimation::GetCurveInterpMode() const
{
	return static_cast<EDatasmithFacadeCurveInterpMode>( GetDatasmithVisibilityAnimation()->GetCurveInterpMode() );
}

void FDatasmithFacadeVisibilityAnimation::GetFrame(int32 Index, int32& OutFrameNumber, bool& bOutVisible) const
{
	const FDatasmithVisibilityFrameInfo& VisbilityInfo = GetDatasmithVisibilityAnimation()->GetFrame(Index);
	OutFrameNumber = VisbilityInfo.FrameNumber;
	bOutVisible = VisbilityInfo.bVisible;
}

void FDatasmithFacadeVisibilityAnimation::RemoveFrame(int32 Index)
{
	GetDatasmithVisibilityAnimation()->RemoveFrame(Index);
}

void FDatasmithFacadeVisibilityAnimation::SetPropagateToChildren(bool bPropagate)
{
	GetDatasmithVisibilityAnimation()->SetPropagateToChildren(bPropagate);
}

bool FDatasmithFacadeVisibilityAnimation::GetPropagateToChildren() const
{
	return GetDatasmithVisibilityAnimation()->GetPropagateToChildren();
}

// FDatasmithFacadeSubsequenceAnimation

FDatasmithFacadeSubsequenceAnimation::FDatasmithFacadeSubsequenceAnimation(const TCHAR* InName)
	: FDatasmithFacadeSubsequenceAnimation( FDatasmithSceneFactory::CreateSubsequenceAnimation(InName) )
{
}

FDatasmithFacadeSubsequenceAnimation::FDatasmithFacadeSubsequenceAnimation(
	const TSharedRef<IDatasmithSubsequenceAnimationElement>& InInternalAnimation
) : FDatasmithFacadeBaseAnimation( InInternalAnimation )
{
}

TSharedRef<IDatasmithSubsequenceAnimationElement> FDatasmithFacadeSubsequenceAnimation::GetDatasmithSubsequenceAnimation() const
{
	return StaticCastSharedRef<IDatasmithSubsequenceAnimationElement>(InternalDatasmithElement);
}

int32 FDatasmithFacadeSubsequenceAnimation::GetStartTime() const
{
	const FFrameNumber FrameNum = GetDatasmithSubsequenceAnimation()->GetStartTime();
	return FrameNum.Value;
}

void FDatasmithFacadeSubsequenceAnimation::SetStartTime(int32 InFrameNumber)
{
	GetDatasmithSubsequenceAnimation()->SetStartTime(FFrameNumber(InFrameNumber));
}

int32 FDatasmithFacadeSubsequenceAnimation::GetDuration() const
{
	return GetDatasmithSubsequenceAnimation()->GetDuration();
}

void FDatasmithFacadeSubsequenceAnimation::SetDuration(int32 InDuration)
{
	GetDatasmithSubsequenceAnimation()->SetDuration(InDuration);
}

float FDatasmithFacadeSubsequenceAnimation::GetTimeScale() const
{
	return GetDatasmithSubsequenceAnimation()->GetTimeScale();
}

void FDatasmithFacadeSubsequenceAnimation::SetTimeScale(float InTimeScale)
{
	GetDatasmithSubsequenceAnimation()->SetTimeScale(InTimeScale);
}

FDatasmithFacadeLevelSequence* FDatasmithFacadeSubsequenceAnimation::GetNewSubsequence() const
{
	TWeakPtr<IDatasmithLevelSequenceElement> LevelSeq = GetDatasmithSubsequenceAnimation()->GetSubsequence();

	if (LevelSeq.IsValid())
	{
		return new FDatasmithFacadeLevelSequence(LevelSeq.Pin().ToSharedRef());
	}
	return nullptr;
}

void FDatasmithFacadeSubsequenceAnimation::SetSubsequence(FDatasmithFacadeLevelSequence* InSubsequence)
{
	GetDatasmithSubsequenceAnimation()->SetSubsequence(InSubsequence->GetDatasmithLevelSequence());
}

// FDatasmithFacadeLevelSequence

FDatasmithFacadeLevelSequence::FDatasmithFacadeLevelSequence(const TCHAR* InName)
	: FDatasmithFacadeElement(FDatasmithSceneFactory::CreateLevelSequence(InName))
{
}

FDatasmithFacadeLevelSequence::FDatasmithFacadeLevelSequence(
	const TSharedRef<IDatasmithLevelSequenceElement>& InInternalLevelSequence
) : FDatasmithFacadeElement( InInternalLevelSequence )
{
}

TSharedRef<IDatasmithLevelSequenceElement> FDatasmithFacadeLevelSequence::GetDatasmithLevelSequence() const
{
	return StaticCastSharedRef<IDatasmithLevelSequenceElement>( InternalDatasmithElement );
}

const TCHAR* FDatasmithFacadeLevelSequence::GetFile() const
{
	return GetDatasmithLevelSequence()->GetFile();
}

void FDatasmithFacadeLevelSequence::SetFile(const TCHAR* InFile)
{
	GetDatasmithLevelSequence()->SetFile(InFile);
}

void FDatasmithFacadeLevelSequence::GetFileHash(TCHAR OutBuffer[33], size_t BufferSize) const
{
	FString HashString = LexToString(GetDatasmithLevelSequence()->GetFileHash());
	FCString::Strncpy(OutBuffer, *HashString, BufferSize);
}

void FDatasmithFacadeLevelSequence::SetFileHash(const TCHAR* Hash)
{
	FMD5Hash Md5Hash;
	LexFromString(Md5Hash, Hash);

	GetDatasmithLevelSequence()->SetFileHash(Md5Hash);
}

float FDatasmithFacadeLevelSequence::GetFrameRate() const
{
	return GetDatasmithLevelSequence()->GetFrameRate();
}

void FDatasmithFacadeLevelSequence::SetFrameRate(float FramePerSecs)
{
	GetDatasmithLevelSequence()->SetFrameRate(FramePerSecs);
}

void FDatasmithFacadeLevelSequence::AddAnimation(FDatasmithFacadeTransformAnimation* InAnimation)
{
	if (InAnimation)
	{
		GetDatasmithLevelSequence()->AddAnimation(InAnimation->GetDatasmithTransformAnimation());
	}
}

int32 FDatasmithFacadeLevelSequence::GetAnimationsCount() const
{
	return GetDatasmithLevelSequence()->GetAnimationsCount();
}

FDatasmithFacadeTransformAnimation* FDatasmithFacadeLevelSequence::GetNewTransformAnimation(int32 InIndex)
{
	if (TSharedPtr<IDatasmithTransformAnimationElement> AnimElement = StaticCastSharedPtr<IDatasmithTransformAnimationElement>( GetDatasmithLevelSequence()->GetAnimation(InIndex) ))
	{
		return new FDatasmithFacadeTransformAnimation(AnimElement.ToSharedRef());
	}

	return nullptr;
}

FDatasmithFacadeVisibilityAnimation* FDatasmithFacadeLevelSequence::GetNewVisibilityAnimation(int32 InIndex)
{
	if (TSharedPtr<IDatasmithVisibilityAnimationElement> AnimElement = StaticCastSharedPtr<IDatasmithVisibilityAnimationElement>(GetDatasmithLevelSequence()->GetAnimation(InIndex)))
	{
		return new FDatasmithFacadeVisibilityAnimation(AnimElement.ToSharedRef());
	}

	return nullptr;
}

FDatasmithFacadeSubsequenceAnimation* FDatasmithFacadeLevelSequence::GetNewSubsequenceAnimation(int32 InIndex)
{
	if (TSharedPtr<IDatasmithSubsequenceAnimationElement> AnimElement = StaticCastSharedPtr<IDatasmithSubsequenceAnimationElement>(GetDatasmithLevelSequence()->GetAnimation(InIndex)))
	{
		return new FDatasmithFacadeSubsequenceAnimation(AnimElement.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeLevelSequence::RemoveAnimation(FDatasmithFacadeBaseAnimation* InAnimation)
{
	if (InAnimation)
	{
		GetDatasmithLevelSequence()->RemoveAnimation(InAnimation->GetDatasmithBaseAnimation());
	}
}
