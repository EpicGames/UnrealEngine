// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeDecal.h"

FDatasmithFacadeActorDecal::FDatasmithFacadeActorDecal(
	const TCHAR* InElementName
)
	: FDatasmithFacadeActor(FDatasmithSceneFactory::CreateDecalActor(InElementName))
{}

FDatasmithFacadeActorDecal::FDatasmithFacadeActorDecal(
	const TSharedRef<IDatasmithDecalActorElement>& InInternalActor
)
	: FDatasmithFacadeActor(InInternalActor)
{}

TSharedRef<IDatasmithDecalActorElement> FDatasmithFacadeActorDecal::GetDatasmithDecalActorElement() const
{
	return StaticCastSharedRef<IDatasmithDecalActorElement>(InternalDatasmithElement);
}

void FDatasmithFacadeActorDecal::GetDimensions(double& OutX, double& OutY, double& OutZ) const
{
	FVector DecalDim = GetDatasmithDecalActorElement()->GetDimensions();
	OutX = DecalDim.X;
	OutY = DecalDim.Y;
	OutZ = DecalDim.Z;
}

void FDatasmithFacadeActorDecal::SetDimensions(double InX, double InY, double InZ)
{
	GetDatasmithDecalActorElement()->SetDimensions(FVector(InX, InY, InZ));
}

const TCHAR* FDatasmithFacadeActorDecal::GetDecalMaterialPathName() const
{
	return GetDatasmithDecalActorElement()->GetDecalMaterialPathName();
}

void FDatasmithFacadeActorDecal::SetDecalMaterialPathName(const TCHAR* InName)
{
	GetDatasmithDecalActorElement()->SetDecalMaterialPathName(InName);
}

int32 FDatasmithFacadeActorDecal::GetSortOrder() const
{
	return GetDatasmithDecalActorElement()->GetSortOrder();
}

void FDatasmithFacadeActorDecal::SetSortOrder(int32 InSortOrder)
{
	GetDatasmithDecalActorElement()->SetSortOrder(InSortOrder);
}

// class FDatasmithFacadeDecalMaterial

FDatasmithFacadeDecalMaterial::FDatasmithFacadeDecalMaterial(
	const TSharedRef<IDatasmithDecalMaterialElement>& InMaterialRef
)
	: FDatasmithFacadeBaseMaterial(InMaterialRef)
{}

FDatasmithFacadeDecalMaterial::FDatasmithFacadeDecalMaterial(
	const TCHAR* InElementName
)
	: FDatasmithFacadeBaseMaterial(FDatasmithSceneFactory::CreateDecalMaterial(InElementName))
{}

TSharedRef<IDatasmithDecalMaterialElement> FDatasmithFacadeDecalMaterial::GetDatasmithDecalMaterial() const
{
	return StaticCastSharedRef<IDatasmithDecalMaterialElement>(InternalDatasmithElement);
}

const TCHAR* FDatasmithFacadeDecalMaterial::GetDiffuseTexturePathName() const
{
	return GetDatasmithDecalMaterial()->GetDiffuseTexturePathName();
}

void FDatasmithFacadeDecalMaterial::SetDiffuseTexturePathName(const TCHAR* DiffuseTexturePathName)
{
	GetDatasmithDecalMaterial()->SetDiffuseTexturePathName(DiffuseTexturePathName);
}

const TCHAR* FDatasmithFacadeDecalMaterial::GetNormalTexturePathName() const
{
	return GetDatasmithDecalMaterial()->GetNormalTexturePathName();
}

void FDatasmithFacadeDecalMaterial::SetNormalTexturePathName(const TCHAR* NormalTexturePathName)
{
	GetDatasmithDecalMaterial()->SetNormalTexturePathName(NormalTexturePathName);
}
