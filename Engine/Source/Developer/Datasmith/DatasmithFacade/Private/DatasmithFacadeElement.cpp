// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeElement.h"


FDatasmithFacadeElement::ECoordinateSystemType FDatasmithFacadeElement::WorldCoordinateSystemType = FDatasmithFacadeElement::ECoordinateSystemType::LeftHandedZup;

float FDatasmithFacadeElement::WorldUnitScale = 1.0;

void FDatasmithFacadeElement::SetCoordinateSystemType(
	ECoordinateSystemType InWorldCoordinateSystemType
)
{
	WorldCoordinateSystemType = InWorldCoordinateSystemType;
}

void FDatasmithFacadeElement::SetWorldUnitScale(
	float InWorldUnitScale
)
{
	WorldUnitScale = FMath::IsNearlyZero(InWorldUnitScale) ? SMALL_NUMBER : InWorldUnitScale;
}

FDatasmithFacadeElement::FDatasmithFacadeElement(
	const TSharedRef<IDatasmithElement>& InElement
)
	: InternalDatasmithElement(InElement)
{}

void FDatasmithFacadeElement::GetStringHash(const TCHAR* InString, TCHAR OutBuffer[33], size_t BufferSize)
{
	FString HashedName = FMD5::HashAnsiString(InString);
	FCString::Strncpy(OutBuffer, *HashedName, BufferSize);
}

void FDatasmithFacadeElement::SetName(
	const TCHAR* InElementName
)
{
	InternalDatasmithElement->SetName(InElementName);
}

const TCHAR* FDatasmithFacadeElement::GetName() const
{
	return InternalDatasmithElement->GetName();
}

void FDatasmithFacadeElement::SetLabel(
	const TCHAR* InElementLabel
)
{
	InternalDatasmithElement->SetLabel(InElementLabel);
}

const TCHAR* FDatasmithFacadeElement::GetLabel() const
{
	return InternalDatasmithElement->GetLabel();
}

template<bool kIsForward, bool kIsDirection, typename Vec_t>
Vec_t FDatasmithFacadeElement::Convert(const Vec_t& V)
{
	Vec_t Tmp;
	switch (WorldCoordinateSystemType)
	{
		case ECoordinateSystemType::LeftHandedYup:
			Tmp = kIsForward ? Vec_t{V.X, -V.Z, V.Y} : Vec_t{V.X, V.Z, -V.Y};
			break;

		case ECoordinateSystemType::RightHandedZup:
			// To avoid perturbating X, which is forward in Unreal, the handedness conversion is done by flipping the side vector Y.
			Tmp = Vec_t{V.X, -V.Y, V.Z};
			break;

		case ECoordinateSystemType::LeftHandedZup:
		default:
			Tmp = V;
			break;
	}

	// World scaling for positions conversions
	if constexpr (!kIsDirection)
	{
		Tmp *= kIsForward ? WorldUnitScale : 1 / WorldUnitScale;
	}

	return Tmp;
}

static constexpr bool kForward = true;
static constexpr bool kBackward = false;
static constexpr bool kDir = true;
static constexpr bool kPos = false;

FVector   FDatasmithFacadeElement::ConvertDirection    (const FVector& In)                  { return Convert<kForward,  kDir>(In);                       }
FVector   FDatasmithFacadeElement::ConvertBackDirection(const FVector& In)                  { return Convert<kBackward, kDir>(In);                       }
FVector   FDatasmithFacadeElement::ConvertPosition     (const FVector& In)                  { return Convert<kForward,  kPos>(In);                       }
FVector   FDatasmithFacadeElement::ConvertBackPosition (const FVector& In)                  { return Convert<kBackward, kPos>(In);                       }

FVector   FDatasmithFacadeElement::ConvertDirection    (double InX, double InY, double InZ) { return Convert<kForward,  kDir>(FVector{InX, InY, InZ});   }
FVector   FDatasmithFacadeElement::ConvertBackDirection(double InX, double InY, double InZ) { return Convert<kBackward, kDir>(FVector{InX, InY, InZ});   }
FVector   FDatasmithFacadeElement::ConvertPosition     (double InX, double InY, double InZ) { return Convert<kForward,  kPos>(FVector{InX, InY, InZ});   }
FVector   FDatasmithFacadeElement::ConvertBackPosition (double InX, double InY, double InZ) { return Convert<kBackward, kPos>(FVector{InX, InY, InZ});   }

FVector3f FDatasmithFacadeElement::ConvertDirection    (const FVector3f& In)                { return Convert<kForward,  kDir>(In);                       }
FVector3f FDatasmithFacadeElement::ConvertBackDirection(const FVector3f& In)                { return Convert<kBackward, kDir>(In);                       }
FVector3f FDatasmithFacadeElement::ConvertPosition     (const FVector3f& In)                { return Convert<kForward,  kPos>(In);                       }
FVector3f FDatasmithFacadeElement::ConvertBackPosition (const FVector3f& In)                { return Convert<kBackward, kPos>(In);                       }

FVector3f FDatasmithFacadeElement::ConvertDirection    (float InX, float InY, float InZ)    { return Convert<kForward,  kDir>(FVector3f{InX, InY, InZ}); }
FVector3f FDatasmithFacadeElement::ConvertBackDirection(float InX, float InY, float InZ)    { return Convert<kBackward, kDir>(FVector3f{InX, InY, InZ}); }
FVector3f FDatasmithFacadeElement::ConvertPosition     (float InX, float InY, float InZ)    { return Convert<kForward,  kPos>(FVector3f{InX, InY, InZ}); }
FVector3f FDatasmithFacadeElement::ConvertBackPosition (float InX, float InY, float InZ)    { return Convert<kBackward, kPos>(FVector3f{InX, InY, InZ}); }


void FDatasmithFacadeElement::ExportAsset(
	FString const& InAssetFolder
)
{
	// By default, there is no Datasmith scene element asset to build and export.
}
