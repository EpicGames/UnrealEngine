// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeKeyValueProperty.h"

FDatasmithFacadeKeyValueProperty::FDatasmithFacadeKeyValueProperty(const TCHAR* InName)
	: FDatasmithFacadeElement(FDatasmithSceneFactory::CreateKeyValueProperty(InName))
{}

FDatasmithFacadeKeyValueProperty::FDatasmithFacadeKeyValueProperty(const TSharedRef<IDatasmithKeyValueProperty>& InKeyValueProperty)
	: FDatasmithFacadeElement(InKeyValueProperty)
{}