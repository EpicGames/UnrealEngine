// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeLog.h"

// Datasmith SDK.
#include "Misc/FileHelper.h"


DEFINE_LOG_CATEGORY(LogDatasmithFacade);

FDatasmithFacadeLog::FDatasmithFacadeLog() :
	LineIndentation(0)
{
}

void FDatasmithFacadeLog::AddLine(
	const TCHAR* InLine
)
{
	UE_LOGF(LogDatasmithFacade, Warning, "Facade: %ls", InLine);

	Log.Append(FString::ChrN(LineIndentation, TEXT('\t')));
	Log.Append(InLine);
	Log.AppendChar(TEXT('\n'));
}

void FDatasmithFacadeLog::MoreIndentation()
{
	LineIndentation += 1;
}

void FDatasmithFacadeLog::LessIndentation()
{
	LineIndentation -= 1;

	if (LineIndentation < 0)
	{
		LineIndentation = 0;
	}
}

void FDatasmithFacadeLog::WriteFile(
	const TCHAR* InFilePath
) const
{
	FFileHelper::SaveStringToFile(Log, InFilePath);
}
