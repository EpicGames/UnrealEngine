// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeMaterial.h"

#include "DatasmithFacadeKeyValueProperty.h"
#include "DatasmithFacadeScene.h"
#include "DatasmithFacadeUEPbrMaterial.h"
#include "DatasmithFacadeDecal.h"

#include "DatasmithDefinitions.h"
#include "DatasmithUtils.h"
#include "Misc/Paths.h"

FDatasmithFacadeBaseMaterial::FDatasmithFacadeBaseMaterial(
	const TSharedRef<IDatasmithBaseMaterialElement>& BaseMaterialElement
) : 
	FDatasmithFacadeElement( BaseMaterialElement )
{
}

FDatasmithFacadeBaseMaterial::EDatasmithMaterialType FDatasmithFacadeBaseMaterial::GetDatasmithMaterialType() const
{
	return GetDatasmithMaterialType(GetDatasmithBaseMaterial());
}

FDatasmithFacadeBaseMaterial::EDatasmithMaterialType FDatasmithFacadeBaseMaterial::GetDatasmithMaterialType(
	const TSharedRef<IDatasmithBaseMaterialElement>& InMaterial
)
{
	if (InMaterial->IsA( EDatasmithElementType::UEPbrMaterial ))
	{
		return EDatasmithMaterialType::UEPbrMaterial;
	}
	else if (InMaterial->IsA(EDatasmithElementType::MaterialInstance))
	{
		return EDatasmithMaterialType::MaterialInstance;
	}
	else if (InMaterial->IsA(EDatasmithElementType::DecalMaterial))
	{
		return EDatasmithMaterialType::DecalMaterial;
	}

	return EDatasmithMaterialType::Unsupported;
}

TSharedRef<IDatasmithBaseMaterialElement> FDatasmithFacadeBaseMaterial::GetDatasmithBaseMaterial() const
{
	return StaticCastSharedRef<IDatasmithBaseMaterialElement>( InternalDatasmithElement );
}

FDatasmithFacadeBaseMaterial* FDatasmithFacadeBaseMaterial::GetNewFacadeBaseMaterialFromSharedPtr(
	const TSharedPtr<IDatasmithBaseMaterialElement>& InMaterial
)
{
	if (InMaterial)
	{
		TSharedRef<IDatasmithBaseMaterialElement> MaterialRef = InMaterial.ToSharedRef();
		EDatasmithMaterialType MaterialType = GetDatasmithMaterialType(MaterialRef);

		switch (MaterialType)
		{
		case EDatasmithMaterialType::UEPbrMaterial:
			return new FDatasmithFacadeUEPbrMaterial(StaticCastSharedRef<IDatasmithUEPbrMaterialElement>(MaterialRef));
		case EDatasmithMaterialType::MaterialInstance:
			return new FDatasmithFacadeMaterialInstance(StaticCastSharedRef<IDatasmithMaterialInstanceElement>(MaterialRef));
		case EDatasmithMaterialType::DecalMaterial:
			return new FDatasmithFacadeDecalMaterial(StaticCastSharedRef<IDatasmithDecalMaterialElement>(MaterialRef));
		case EDatasmithMaterialType::Unsupported:
		default:
			return nullptr;
		}
	}

	return nullptr;
}

FDatasmithFacadeMaterialInstance::FDatasmithFacadeMaterialInstance(const TCHAR* InElementName) 
	: FDatasmithFacadeBaseMaterial( FDatasmithSceneFactory::CreateMaterialInstance(InElementName))
{
	TSharedPtr<IDatasmithMaterialInstanceElement> MaterialInstance = GetDatasmithMaterialInstance();
	MaterialInstance->SetMaterialType(EDatasmithReferenceMaterialType::Opaque);
}

FDatasmithFacadeMaterialInstance::FDatasmithFacadeMaterialInstance(const TSharedRef<IDatasmithMaterialInstanceElement>& InMaterialRef)
	: FDatasmithFacadeBaseMaterial(InMaterialRef)
{}

FDatasmithFacadeMaterialInstance::EMaterialInstanceType FDatasmithFacadeMaterialInstance::GetMaterialType() const
{
	return static_cast<EMaterialInstanceType>(GetDatasmithMaterialInstance()->GetMaterialType());
}

void FDatasmithFacadeMaterialInstance::SetMaterialType(
	EMaterialInstanceType InMaterialInstanceType
)
{
	GetDatasmithMaterialInstance()->SetMaterialType(static_cast<EDatasmithReferenceMaterialType>(InMaterialInstanceType));
}

FDatasmithFacadeMaterialInstance::EMaterialInstanceQuality FDatasmithFacadeMaterialInstance::GetQuality() const
{
	return static_cast<EMaterialInstanceQuality>(GetDatasmithMaterialInstance()->GetQuality());
}

void FDatasmithFacadeMaterialInstance::SetQuality(
	EMaterialInstanceQuality InQuality
)
{
	GetDatasmithMaterialInstance()->SetQuality(static_cast<EDatasmithReferenceMaterialQuality>(InQuality));
}

const TCHAR* FDatasmithFacadeMaterialInstance::GetCustomMaterialPathName() const
{
	return GetDatasmithMaterialInstance()->GetCustomMaterialPathName();
}

void FDatasmithFacadeMaterialInstance::SetCustomMaterialPathName(
	const TCHAR* InPathName
)
{
	GetDatasmithMaterialInstance()->SetCustomMaterialPathName(InPathName);
}

void FDatasmithFacadeMaterialInstance::AddColor(
	const TCHAR*  InPropertyName,
	unsigned char InR,
	unsigned char InG,
	unsigned char InB,
	unsigned char InA
)
{
	// Convert the sRGBA color to a Datasmith linear color.
	FLinearColor LinearColor(FColor(InR, InG, InB, InA));

	// Add the Datasmith material linear color property.
	AddColor(InPropertyName, LinearColor.R, LinearColor.G, LinearColor.B, LinearColor.A);
}

void FDatasmithFacadeMaterialInstance::AddColor(
	const TCHAR* InPropertyName,
	float        InR,
	float        InG,
	float        InB,
	float        InA
)
{
	FLinearColor LinearColor(InR, InG, InB, InA);

	// Create a new Datasmith material color property.
	TSharedPtr<IDatasmithKeyValueProperty> MaterialPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
	MaterialPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Color);
	MaterialPropertyPtr->SetValue(*LinearColor.ToString());

	// Add the new property to the Datasmith material properties.
	GetDatasmithMaterialInstance()->AddProperty(MaterialPropertyPtr);
}

void FDatasmithFacadeMaterialInstance::AddTexture(
	const TCHAR* InPropertyName,
	const FDatasmithFacadeTexture* InTexture
)
{
	if (InTexture)
	{
		// Create a new Datasmith material texture property.
		TSharedPtr<IDatasmithKeyValueProperty> MaterialPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
		MaterialPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Texture);
		MaterialPropertyPtr->SetValue(InTexture->GetName());

		// Add the new property to the Datasmith material properties.
		GetDatasmithMaterialInstance()->AddProperty(MaterialPropertyPtr);
	}
}

void FDatasmithFacadeMaterialInstance::AddString(
	const TCHAR* InPropertyName,
	const TCHAR* InPropertyValue
)
{
	if (!FString(InPropertyValue).IsEmpty())
	{
		// Create a new Datasmith material string property.
		TSharedPtr<IDatasmithKeyValueProperty> MaterialPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
		MaterialPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::String);
		MaterialPropertyPtr->SetValue(InPropertyValue);

		// Add the new property to the array of Datasmith material properties.
		GetDatasmithMaterialInstance()->AddProperty(MaterialPropertyPtr);
	}
}

void FDatasmithFacadeMaterialInstance::AddFloat(
	const TCHAR* InPropertyName,
	float        InPropertyValue
)
{
	// Create a new Datasmith material float property.
	TSharedPtr<IDatasmithKeyValueProperty> MaterialPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
	MaterialPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Float);
	MaterialPropertyPtr->SetValue(*FString::Printf(TEXT("%f"), InPropertyValue));

	// Add the new property to the Datasmith material properties.
	GetDatasmithMaterialInstance()->AddProperty(MaterialPropertyPtr);
}

void FDatasmithFacadeMaterialInstance::AddBoolean(
	const TCHAR* InPropertyName,
	bool         bInPropertyValue
)
{
	// Create a new Datasmith material boolean property.
	TSharedPtr<IDatasmithKeyValueProperty> MaterialPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
	MaterialPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Bool);
	MaterialPropertyPtr->SetValue(bInPropertyValue ? TEXT("True") : TEXT("False"));

	// Add the new property to the Datasmith material properties.
	GetDatasmithMaterialInstance()->AddProperty(MaterialPropertyPtr);
}

int32 FDatasmithFacadeMaterialInstance::GetPropertiesCount() const
{
	return GetDatasmithMaterialInstance()->GetPropertiesCount();
}

FDatasmithFacadeKeyValueProperty* FDatasmithFacadeMaterialInstance::GetNewProperty(
	int32 PropertyIndex
) const
{
	if (const TSharedPtr<IDatasmithKeyValueProperty>& Property = GetDatasmithMaterialInstance()->GetProperty(PropertyIndex))
	{
		return new FDatasmithFacadeKeyValueProperty(Property.ToSharedRef());
	}
	else
	{
		return nullptr;
	}
}

FDatasmithFacadeKeyValueProperty* FDatasmithFacadeMaterialInstance::GetNewPropertyByName(
	const TCHAR* PropertyName
) const
{
	if (const TSharedPtr<IDatasmithKeyValueProperty>& Property = GetDatasmithMaterialInstance()->GetPropertyByName(PropertyName))
	{
		return new FDatasmithFacadeKeyValueProperty(Property.ToSharedRef());
	}
	else
	{
		return nullptr;
	}
}

TSharedRef<IDatasmithMaterialInstanceElement> FDatasmithFacadeMaterialInstance::GetDatasmithMaterialInstance() const
{
	return StaticCastSharedRef<IDatasmithMaterialInstanceElement>( InternalDatasmithElement );
}