// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeMaterialID.h"

#include "DatasmithFacadeScene.h"

FDatasmithFacadeMaterialID::FDatasmithFacadeMaterialID(
	const TCHAR* MaterialIDName
)
	: FDatasmithFacadeElement(FDatasmithSceneFactory::CreateMaterialId(MaterialIDName))
{}

FDatasmithFacadeMaterialID::FDatasmithFacadeMaterialID(
	const TSharedRef<IDatasmithMaterialIDElement>& InMaterialElement
)
	: FDatasmithFacadeElement(InMaterialElement)
{}

void FDatasmithFacadeMaterialID::SetId(
	int32 Id
)
{
	GetMaterialIDElement()->SetId(Id);
}

int32 FDatasmithFacadeMaterialID::GetId() const
{
	return GetMaterialIDElement()->GetId();
}

TSharedRef<IDatasmithMaterialIDElement> FDatasmithFacadeMaterialID::GetMaterialIDElement() const
{
	return StaticCastSharedRef<IDatasmithMaterialIDElement>(InternalDatasmithElement);
}
