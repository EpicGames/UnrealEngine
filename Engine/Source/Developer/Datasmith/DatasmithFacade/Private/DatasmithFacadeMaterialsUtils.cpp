// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeMaterialsUtils.h"
#include "DatasmithUtils.h"
#include "Misc/Paths.h"

void FDatasmithFacadeMaterialsUtils::FUVEditParameters::SetUVOffset( float X, float Y )
{
	UVEditParameters.UVOffset = FVector2D( X, Y );
}

void FDatasmithFacadeMaterialsUtils::FUVEditParameters::GetUVOffset( float& OutX, float& OutY ) const
{
	OutX = (float)UVEditParameters.UVOffset.X;		// LWC_TODO: Precision loss
	OutY = (float)UVEditParameters.UVOffset.Y;
}

void FDatasmithFacadeMaterialsUtils::FUVEditParameters::SetUVTiling( float X, float Y )
{
	UVEditParameters.UVTiling = FVector2D( X, Y );
}

void FDatasmithFacadeMaterialsUtils::FUVEditParameters::GetUVTiling( float& OutX, float& OutY ) const
{
	OutX = (float)UVEditParameters.UVTiling.X;		// LWC_TODO: Precision loss
	OutY = (float)UVEditParameters.UVTiling.Y;
}

void FDatasmithFacadeMaterialsUtils::FUVEditParameters::SetRotationPivot( float X, float Y, float Z )
{
	UVEditParameters.RotationPivot = FVector( X, Y, Z );
}

void FDatasmithFacadeMaterialsUtils::FUVEditParameters::GetRotationPivot( float& OutX, float& OutY, float& OutZ ) const
{
	OutX = (float)UVEditParameters.RotationPivot.X;		// LWC_TODO: Precision loss
	OutY = (float)UVEditParameters.RotationPivot.Y;
	OutZ = (float)UVEditParameters.RotationPivot.Z;
}

void FDatasmithFacadeMaterialsUtils::FUVEditParameters::SetRotationAngle( float Angle )
{
	UVEditParameters.RotationAngle = Angle;
}

float FDatasmithFacadeMaterialsUtils::FUVEditParameters::GetRotationAngle() const
{
	return UVEditParameters.RotationAngle;
}

void FDatasmithFacadeMaterialsUtils::FUVEditParameters::SetChannelIndex( uint8 ChannelIndex )
{
	UVEditParameters.ChannelIndex = ChannelIndex;
}

uint8 FDatasmithFacadeMaterialsUtils::FUVEditParameters::GetChannelIndex() const
{
	return UVEditParameters.ChannelIndex;
}

void FDatasmithFacadeMaterialsUtils::FDatasmithFacadeMaterialsUtils::FUVEditParameters::SetIsUsingRealWorldScale( bool bIsUsingRealWorldScale )
{
	UVEditParameters.bIsUsingRealWorldScale = bIsUsingRealWorldScale;
}

bool FDatasmithFacadeMaterialsUtils::FUVEditParameters::GetIsUsingRealWorldScale() const
{
	return UVEditParameters.bIsUsingRealWorldScale;
}

void FDatasmithFacadeMaterialsUtils::FUVEditParameters::SetMirrorU( bool bMirrorU )
{
	UVEditParameters.bMirrorU = bMirrorU;
}

bool FDatasmithFacadeMaterialsUtils::FUVEditParameters::GetMirrorU() const
{
	return UVEditParameters.bMirrorU;
}

void FDatasmithFacadeMaterialsUtils::FUVEditParameters::SetMirrorV( bool bMirrorV )
{
	UVEditParameters.bMirrorV = bMirrorV;
}

bool FDatasmithFacadeMaterialsUtils::FUVEditParameters::GetMirrorV() const
{
	return UVEditParameters.bMirrorV;
}

void FDatasmithFacadeMaterialsUtils::FWeightedMaterialExpressionParameters::SetColor( float R, float G, float B, float A )
{
	Color = FLinearColor( R, G, B, A );
}

void FDatasmithFacadeMaterialsUtils::FWeightedMaterialExpressionParameters::SetColorsRGB( uint8 R, uint8 G, uint8 B, uint8 A )
{
	Color = FLinearColor::FromSRGBColor( FColor( R, G, B, A ) );
}

void FDatasmithFacadeMaterialsUtils::FWeightedMaterialExpressionParameters::SetScalar( float Value )
{
	Scalar = Value;
}

void FDatasmithFacadeMaterialsUtils::FWeightedMaterialExpressionParameters::SetTextureMode( FDatasmithFacadeTexture::ETextureMode InTextureMode )
{
	TextureMode = InTextureMode;
}

void FDatasmithFacadeMaterialsUtils::FWeightedMaterialExpressionParameters::SetExpression( const FDatasmithFacadeMaterialExpression& InExpression )
{
	Expression = InExpression;
}


FDatasmithFacadeMaterialExpressionTexture* FDatasmithFacadeMaterialsUtils::CreateNewFacadeTextureExpression( FDatasmithFacadeUEPbrMaterial& MaterialElement, const TCHAR* ParameterName, const TCHAR* TextureMapPath, const FUVEditParameters& UVParameters )
{
	TSharedRef<IDatasmithUEPbrMaterialElement> MaterialElementRef = StaticCastSharedRef<IDatasmithUEPbrMaterialElement>( MaterialElement.GetDatasmithBaseMaterial() );
	IDatasmithMaterialExpressionTexture* Expression = DatasmithMaterialsUtils::CreateTextureExpression( MaterialElementRef, ParameterName, TextureMapPath, UVParameters.UVEditParameters );

	if ( Expression )
	{
		return new FDatasmithFacadeMaterialExpressionTexture( Expression, MaterialElementRef );
	}

	return nullptr;
}

FDatasmithFacadeMaterialExpression* FDatasmithFacadeMaterialsUtils::CreateNewFacadeWeightedMaterialExpression( FDatasmithFacadeUEPbrMaterial& MaterialElement, const TCHAR* ParameterName, FWeightedMaterialExpressionParameters& WeightedExpressionParameter )
{
	TSharedRef<IDatasmithUEPbrMaterialElement> MaterialElementRef = StaticCastSharedRef<IDatasmithUEPbrMaterialElement>( MaterialElement.GetDatasmithBaseMaterial() );
	TOptional<FLinearColor>& ColorParam = WeightedExpressionParameter.Color;
	TOptional<float>& ScalarParam = WeightedExpressionParameter.Scalar;
	IDatasmithMaterialExpression* Expression = WeightedExpressionParameter.Expression.GetMaterialExpression();
	float& Weight = WeightedExpressionParameter.Weight;
	EDatasmithTextureMode TextureMode = static_cast<EDatasmithTextureMode>( WeightedExpressionParameter.TextureMode );

	IDatasmithMaterialExpression* ResultExpression = DatasmithMaterialsUtils::CreateWeightedMaterialExpression( MaterialElementRef, ParameterName, ColorParam, ScalarParam, Expression, Weight, TextureMode );
	if ( ResultExpression )
	{
		return new FDatasmithFacadeMaterialExpression( ResultExpression, MaterialElementRef );
	}

	return nullptr;
}

FDatasmithFacadeTexture* FDatasmithFacadeMaterialsUtils::CreateSimpleTextureElement( const TCHAR* InTextureFilePath, FDatasmithFacadeTexture::ETextureMode InTextureMode )
{
	if (!FString( InTextureFilePath ).IsEmpty())
	{
		// Sanitize full path before hashing: this ensures back/forward slashes will result in the 
		// same converted file path (as they are refering to the same file).
#if PLATFORM_WINDOWS
		const FString ConvertedFilePath = FDatasmithUtils::SanitizeObjectName( FString( InTextureFilePath ).ToLower() );
#else
		const FString ConvertedFilePath = FDatasmithUtils::SanitizeObjectName( FString( InTextureFilePath ) );
#endif

		// Hash file path and use that as texture name.
		const FString HashedFilePath = FMD5::HashAnsiString( *ConvertedFilePath );

		FString TextureName = FString::Printf( TEXT( "%ls_%d" ), *HashedFilePath, int( InTextureMode ) );

		// Create a Datasmith texture element.
		TSharedRef<IDatasmithTextureElement> TexturePtr = FDatasmithSceneFactory::CreateTexture( *TextureName );

		// Set the texture label used in the Unreal UI.
		TexturePtr->SetLabel( *FPaths::GetBaseFilename( InTextureFilePath ) );

		// Set the Datasmith texture mode.
		TexturePtr->SetTextureMode( static_cast<EDatasmithTextureMode>( InTextureMode ) );

		// Set the Datasmith texture file path.
		TexturePtr->SetFile( InTextureFilePath );

		return new FDatasmithFacadeTexture( TexturePtr );
	}

	return nullptr;
}