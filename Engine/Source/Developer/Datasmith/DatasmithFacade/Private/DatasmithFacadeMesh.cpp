// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeMesh.h"

#include "DatasmithFacadeScene.h"
#include "DatasmithFacadeMaterialID.h"

// Datasmith SDK.
#include "DatasmithMesh.h"
#include "DatasmithMeshExporter.h"

void FDatasmithFacadeMesh::CalculateHash(TCHAR OutBuffer[33], size_t BufferSize) const
{
	FString HashString = LexToString(GetDatasmithMesh().CalculateHash());
	FCString::Strncpy(OutBuffer, *HashString, BufferSize);
}

void FDatasmithFacadeMesh::SetVertex(int32 Index, float X, float Y, float Z)
{
	FVector3f Position(FDatasmithFacadeElement::ConvertPosition(X, Y, Z));
	GetDatasmithMesh().SetVertex(Index, Position.X, Position.Y, Position.Z);
}

void FDatasmithFacadeMesh::GetVertex(int32 Index, float& OutX, float& OutY, float& OutZ) const
{
	FVector3f Position = FDatasmithFacadeElement::ConvertBackPosition(GetDatasmithMesh().GetVertex(Index));
	OutX = Position.X;
	OutY = Position.Y;
	OutZ = Position.Z;
}

void FDatasmithFacadeMesh::SetNormal(int32 Index, float X, float Y, float Z)
{
	FVector3f Normal(FDatasmithFacadeElement::ConvertDirection(X, Y, Z));
	GetDatasmithMesh().SetNormal(Index, Normal.X, Normal.Y, Normal.Z);
}

void FDatasmithFacadeMesh::GetNormal(int32 Index, float& OutX, float& OutY, float& OutZ) const
{
	FVector3f Normal(FDatasmithFacadeElement::ConvertBackDirection(GetDatasmithMesh().GetNormal(Index)));
	OutX = Normal.X;
	OutY = Normal.Y;
	OutZ = Normal.Z;
}

FDatasmithFacadeMeshElement::FDatasmithFacadeMeshElement(
	const TCHAR* InElementName
)
	: FDatasmithFacadeElement(FDatasmithSceneFactory::CreateMesh(InElementName))
{}

FDatasmithFacadeMeshElement::FDatasmithFacadeMeshElement(
	const TSharedRef<IDatasmithMeshElement>& InMeshElement
)
	: FDatasmithFacadeElement(InMeshElement)
{}

void FDatasmithFacadeMeshElement::GetFileHash(TCHAR OutBuffer[33], size_t BufferSize) const
{
	FString HashString = LexToString(GetDatasmithMeshElement()->GetFileHash());
	FCString::Strncpy(OutBuffer, *HashString, BufferSize);
}

void FDatasmithFacadeMeshElement::SetFileHash(const TCHAR* Hash)
{
	FMD5Hash Md5Hash;
	LexFromString(Md5Hash, Hash);

	GetDatasmithMeshElement()->SetFileHash(Md5Hash);
}

FDatasmithFacadeMaterialID* FDatasmithFacadeMeshElement::GetMaterialSlotAt(int32 Index)
{
	TSharedPtr<IDatasmithMaterialIDElement> MaterialID = GetDatasmithMeshElement()->GetMaterialSlotAt(Index);
	return MaterialID.IsValid() ? new FDatasmithFacadeMaterialID(MaterialID.ToSharedRef()) : nullptr;
}
