// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeMetaData.h"

#include "DatasmithFacadeScene.h"
#include "DatasmithFacadeKeyValueProperty.h"

FDatasmithFacadeMetaData::FDatasmithFacadeMetaData(
	const TCHAR* InElementName
)
	: FDatasmithFacadeElement(FDatasmithSceneFactory::CreateMetaData(InElementName))
{
}

FDatasmithFacadeMetaData::FDatasmithFacadeMetaData(
	const TSharedRef<IDatasmithMetaDataElement>& InMetaDataElement
)
	: FDatasmithFacadeElement(InMetaDataElement)
{
}

void FDatasmithFacadeMetaData::AddPropertyBoolean(
	const TCHAR* InPropertyName,
	bool bInPropertyValue
)
{
	// Create a new Datasmith metadata boolean property.
	TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
	MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Bool);
	MetadataPropertyPtr->SetValue(bInPropertyValue ? TEXT("True") : TEXT("False"));

	// Add the new property to the array of Datasmith metadata properties.
	GetDatasmithMetaDataElement()->AddProperty(MetadataPropertyPtr);
}

void FDatasmithFacadeMetaData::AddPropertyColor(
	const TCHAR* InPropertyName,
	uint8 InR,
	uint8 InG,
	uint8 InB,
	uint8 InA
)
{
	// Convert the sRGBA color to a Datasmith linear color.
	FColor       Color(InR, InG, InB, InA);
	FLinearColor LinearColor(Color);

	// Create a new Datasmith metadata color property.
	TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
	MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Color);
	MetadataPropertyPtr->SetValue(*LinearColor.ToString());

	// Add the new property to the array of Datasmith metadata properties.
	GetDatasmithMetaDataElement()->AddProperty(MetadataPropertyPtr);
}

void FDatasmithFacadeMetaData::AddPropertyFloat(
	const TCHAR* InPropertyName,
	float InPropertyValue
)
{
	// Create a new Datasmith metadata float property.
	TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
	MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Float);
	MetadataPropertyPtr->SetValue(*FString::Printf(TEXT("%f"), InPropertyValue));

	// Add the new property to the array of Datasmith metadata properties.
	GetDatasmithMetaDataElement()->AddProperty(MetadataPropertyPtr);
}

void FDatasmithFacadeMetaData::AddPropertyString(
	const TCHAR* InPropertyName,
	const TCHAR* InPropertyValue
)
{
	// Create a new Datasmith metadata string property.
	TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
	MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::String);
	MetadataPropertyPtr->SetValue(InPropertyValue);

	// Add the new property to the array of Datasmith metadata properties.
	GetDatasmithMetaDataElement()->AddProperty(MetadataPropertyPtr);
}

void FDatasmithFacadeMetaData::AddPropertyTexture(
	const TCHAR* InPropertyName,
	const TCHAR* InTextureFilePath
)
{
	// Create a new Datasmith metadata texture property.
	TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
	MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Texture);
	MetadataPropertyPtr->SetValue(InTextureFilePath);

	// Add the new property to the array of Datasmith metadata properties.
	GetDatasmithMetaDataElement()->AddProperty(MetadataPropertyPtr);
}

void FDatasmithFacadeMetaData::AddPropertyVector(
	const TCHAR* InPropertyName,
	const TCHAR* InPropertyValue
)
{
	// Create a new Datasmith metadata vector property.
	TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(InPropertyName);
	MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Vector);
	MetadataPropertyPtr->SetValue(InPropertyValue);

	// Add the new property to the array of Datasmith metadata properties.
	GetDatasmithMetaDataElement()->AddProperty(MetadataPropertyPtr);
}

void FDatasmithFacadeMetaData::AddProperty(
	const FDatasmithFacadeKeyValueProperty* InProperty
)
{
	if (InProperty)
	{
		GetDatasmithMetaDataElement()->AddProperty(InProperty->GetDatasmithKeyValueProperty());
	}
}

int32 FDatasmithFacadeMetaData::GetPropertiesCount() const
{
	return GetDatasmithMetaDataElement()->GetPropertiesCount();
}

FDatasmithFacadeKeyValueProperty* FDatasmithFacadeMetaData::GetNewProperty(
	int32 PropertyIndex
) const
{
	if (const TSharedPtr<IDatasmithKeyValueProperty>& Property = GetDatasmithMetaDataElement()->GetProperty(PropertyIndex))
	{
		return new FDatasmithFacadeKeyValueProperty(Property.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeMetaData::SetAssociatedElement(
	const FDatasmithFacadeElement* Element
)
{
	TSharedRef<IDatasmithMetaDataElement> MetaDataElement = GetDatasmithMetaDataElement();
	if (Element)
	{
		MetaDataElement->SetAssociatedElement(Element->GetDatasmithElement());
	}
	else
	{
		MetaDataElement->SetAssociatedElement(TSharedPtr<IDatasmithElement>());
	}
}

void FDatasmithFacadeMetaData::RemoveProperty(
	const FDatasmithFacadeKeyValueProperty* Property
)
{
	if (Property)
	{
		GetDatasmithMetaDataElement()->RemoveProperty( Property->GetDatasmithKeyValueProperty() );
	}
}

TSharedRef<IDatasmithMetaDataElement> FDatasmithFacadeMetaData::GetDatasmithMetaDataElement() const
{
	return StaticCastSharedRef<IDatasmithMetaDataElement>(GetDatasmithElement());
}