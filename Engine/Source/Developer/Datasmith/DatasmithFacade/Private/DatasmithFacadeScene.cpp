// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeScene.h"

// Datasmith facade.
#include "DatasmithFacadeActor.h"
#include "DatasmithFacadeElement.h"
#include "DatasmithFacadeMaterial.h"
#include "DatasmithFacadeMesh.h"
#include "DatasmithFacadeMetaData.h"
#include "DatasmithFacadeTexture.h"
#include "DatasmithFacadeVariant.h"
#include "DatasmithFacadeAnimation.h"

// Datasmith SDK.
#include "DatasmithAnimationSerializer.h"
#include "DatasmithExporterManager.h"
#include "DatasmithExportOptions.h"
#include "DatasmithSceneExporter.h"

#include "Misc/Paths.h"
#include "HAL/FileManager.h"

FDatasmithFacadeScene::FDatasmithFacadeScene(
	const TCHAR* InApplicationHostName,
	const TCHAR* InApplicationVendorName,
	const TCHAR* InApplicationProductName,
	const TCHAR* InApplicationProductVersion
) :
	SceneRef(FDatasmithSceneFactory::CreateScene(TEXT(""))),
	SceneExporterRef(MakeShared<FDatasmithSceneExporter>())
{
	// Set the name of the host application used to build the scene.
	SceneRef->SetHost(InApplicationHostName);

	// Set the vendor name of the application used to build the scene.
	SceneRef->SetVendor(InApplicationVendorName);

	// Set the product name of the application used to build the scene.
	SceneRef->SetProductName(InApplicationProductName);

	// Set the product version of the application used to build the scene.
	SceneRef->SetProductVersion(InApplicationProductVersion);
}

void FDatasmithFacadeScene::AddActor(
	FDatasmithFacadeActor* InActorPtr
)
{
	if (InActorPtr)
	{
		SceneRef->AddActor(InActorPtr->GetDatasmithActorElement());
	}
}

int32 FDatasmithFacadeScene::GetActorsCount() const
{
	return SceneRef->GetActorsCount();
}

FDatasmithFacadeActor* FDatasmithFacadeScene::GetNewActor(
	int32 ActorIndex
)
{
	if (TSharedPtr<IDatasmithActorElement> ActorElement = SceneRef->GetActor(ActorIndex))
	{
		return FDatasmithFacadeActor::GetNewFacadeActorFromSharedPtr(ActorElement);
	}

	return nullptr;
}

void FDatasmithFacadeScene::RemoveActor(
	FDatasmithFacadeActor* InActorPtr,
	EActorRemovalRule RemovalRule
)
{
	SceneRef->RemoveActor(InActorPtr->GetDatasmithActorElement(), static_cast<EDatasmithActorRemovalRule>( RemovalRule ));
}

void FDatasmithFacadeScene::AddMaterial(
	FDatasmithFacadeBaseMaterial* InMaterialPtr
)
{
	if (InMaterialPtr)
	{
		SceneRef->AddMaterial(InMaterialPtr->GetDatasmithBaseMaterial());
	}
}

int32 FDatasmithFacadeScene::GetMaterialsCount() const
{
	return SceneRef->GetMaterialsCount();
}

FDatasmithFacadeBaseMaterial* FDatasmithFacadeScene::GetNewMaterial(
	int32 MaterialIndex
)
{
	if (TSharedPtr<IDatasmithBaseMaterialElement> ActorElement = SceneRef->GetMaterial( MaterialIndex ))
	{
		return FDatasmithFacadeBaseMaterial::GetNewFacadeBaseMaterialFromSharedPtr( ActorElement );
	}

	return nullptr;
}

void FDatasmithFacadeScene::RemoveMaterial(
	FDatasmithFacadeBaseMaterial* InMaterialPtr
)
{
	if (InMaterialPtr)
	{
		SceneRef->RemoveMaterial( InMaterialPtr->GetDatasmithBaseMaterial() );
	}
}

FDatasmithFacadeMeshElement* FDatasmithFacadeScene::ExportDatasmithMesh(
	FDatasmithFacadeMesh* Mesh,
	FDatasmithFacadeMesh* CollisionMesh /*= nullptr*/
)
{
	TSharedPtr<IDatasmithMeshElement> ExportedMeshElement;
	FString AssetOutputPath = SceneExporterRef->GetAssetsOutputPath();

	if (Mesh && !AssetOutputPath.IsEmpty())
	{
		FDatasmithMesh& MeshRef = Mesh->GetDatasmithMesh();
		FDatasmithMesh* CollitionMeshPtr = CollisionMesh ? &CollisionMesh->GetDatasmithMesh() : nullptr;
		ExportedMeshElement = FDatasmithMeshExporter().ExportToUObject( *AssetOutputPath, Mesh->GetName(), MeshRef, CollitionMeshPtr, FDatasmithExportOptions::LightmapUV );
	}
	return ExportedMeshElement.IsValid() ? new FDatasmithFacadeMeshElement(ExportedMeshElement.ToSharedRef()) : nullptr;
}

bool FDatasmithFacadeScene::ExportDatasmithMesh(
	FDatasmithFacadeMeshElement* MeshElement,
	FDatasmithFacadeMesh* Mesh,
	FDatasmithFacadeMesh* CollisionMesh /*= nullptr*/
)
{
	TSharedPtr<IDatasmithMeshElement> ExportedMeshElement;
	FString AssetOutputPath = SceneExporterRef->GetAssetsOutputPath();

	if (MeshElement && Mesh && !AssetOutputPath.IsEmpty())
	{
		TSharedPtr<IDatasmithMeshElement> MeshElementSharedPtr = MeshElement->GetDatasmithMeshElement();
		FDatasmithMesh& MeshRef = Mesh->GetDatasmithMesh();
		FDatasmithMesh* CollitionMeshPtr = CollisionMesh ? &CollisionMesh->GetDatasmithMesh() : nullptr;
		return FDatasmithMeshExporter().ExportToUObject(MeshElementSharedPtr, *AssetOutputPath, MeshRef, CollitionMeshPtr, FDatasmithExportOptions::LightmapUV);
	}
	return false;
}

void FDatasmithFacadeScene::AddMesh(
	FDatasmithFacadeMeshElement* InMeshPtr
)
{
	if (InMeshPtr)
	{
		SceneRef->AddMesh(InMeshPtr->GetDatasmithMeshElement());
	}
}

FDatasmithFacadeMeshElement* FDatasmithFacadeScene::GetNewMesh(
	int32 MeshIndex
)
{
	if (const TSharedPtr<IDatasmithMeshElement> DSMesh = SceneRef->GetMesh(MeshIndex))
	{
		return new FDatasmithFacadeMeshElement(DSMesh.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeScene::RemoveMesh(
	FDatasmithFacadeMeshElement* InMeshElementPtr
)
{
	if (InMeshElementPtr)
	{
		SceneRef->RemoveMesh(InMeshElementPtr->GetDatasmithMeshElement());
	}
}

void FDatasmithFacadeScene::AddTexture(
	FDatasmithFacadeTexture* InTexturePtr
)
{
	if (InTexturePtr)
	{
		// Make sure ElementHash is valid
		TSharedRef<IDatasmithTextureElement> TextureElement = InTexturePtr->GetDatasmithTextureElement();

		FMD5Hash FileHash = FMD5Hash::HashFile(TextureElement->GetFile());
		TextureElement->SetFileHash(FileHash);

		TextureElement->CalculateElementHash(true);

		SceneRef->AddTexture(TextureElement);
	}
}

int32 FDatasmithFacadeScene::GetTexturesCount() const
{
	return SceneRef->GetTexturesCount();
}

FDatasmithFacadeTexture* FDatasmithFacadeScene::GetNewTexture(
	int32 TextureIndex
)
{
	if (TSharedPtr<IDatasmithTextureElement> TextureElement = SceneRef->GetTexture(TextureIndex))
	{
		return new FDatasmithFacadeTexture(TextureElement.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeScene::RemoveTexture(
	FDatasmithFacadeTexture* InTexturePtr
)
{
	SceneRef->RemoveTexture(InTexturePtr->GetDatasmithTextureElement());
}

void FDatasmithFacadeScene::AddLevelSequence(
	FDatasmithFacadeLevelSequence* InLevelSequence
)
{
	if (InLevelSequence)
	{
		SceneRef->AddLevelSequence(InLevelSequence->GetDatasmithLevelSequence());
	}
}

int32 FDatasmithFacadeScene::GetLevelSequencesCount() const
{
	return SceneRef->GetLevelSequencesCount();
}

FDatasmithFacadeLevelSequence* FDatasmithFacadeScene::GetNewLevelSequence(
	int32 LevelSequenceIndex
)
{
	if (TSharedPtr<IDatasmithLevelSequenceElement> LevelSequenceElement = SceneRef->GetLevelSequence(LevelSequenceIndex))
	{
		return new FDatasmithFacadeLevelSequence(LevelSequenceElement.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeScene::RemoveLevelSequence(
	FDatasmithFacadeLevelSequence* InLevelSequence
)
{
	SceneRef->RemoveLevelSequence(InLevelSequence->GetDatasmithLevelSequence());
}

void FDatasmithFacadeScene::AddLevelVariantSets(
	FDatasmithFacadeLevelVariantSets* InLevelVariantSetsPtr
)
{
	if (InLevelVariantSetsPtr)
	{
		SceneRef->AddLevelVariantSets(InLevelVariantSetsPtr->GetDatasmithLevelVariantSets());
	}
}

int32 FDatasmithFacadeScene::GetLevelVariantSetsCount() const
{
	return SceneRef->GetLevelVariantSetsCount();
}

FDatasmithFacadeLevelVariantSets* FDatasmithFacadeScene::GetNewLevelVariantSets(
	int32 LevelVariantSetsIndex
)
{
	if (TSharedPtr<IDatasmithLevelVariantSetsElement> LevelVariantSetsElement = SceneRef->GetLevelVariantSets(LevelVariantSetsIndex))
	{
		return new FDatasmithFacadeLevelVariantSets(LevelVariantSetsElement.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeScene::RemoveLevelVariantSets(
	FDatasmithFacadeLevelVariantSets* InLevelVariantSetsPtr
)
{
	SceneRef->RemoveLevelVariantSets(InLevelVariantSetsPtr->GetDatasmithLevelVariantSets());
}

void FDatasmithFacadeScene::AddMetaData(
	FDatasmithFacadeMetaData* InMetaData
)
{
	if (InMetaData)
	{
		SceneRef->AddMetaData(InMetaData->GetDatasmithMetaDataElement());
	}
}

int32 FDatasmithFacadeScene::GetMetaDataCount() const
{
	return SceneRef->GetMetaDataCount();
}

FDatasmithFacadeMetaData* FDatasmithFacadeScene::GetNewMetaData(
	int32 MetaDataIndex
)
{
	if (TSharedPtr<IDatasmithMetaDataElement> MetaDataElement = SceneRef->GetMetaData(MetaDataIndex))
	{
		return new FDatasmithFacadeMetaData(MetaDataElement.ToSharedRef());
	}

	return nullptr;
}

FDatasmithFacadeMetaData* FDatasmithFacadeScene::GetNewMetaData(
	FDatasmithFacadeElement* Element
)
{
	if (Element)
	{
		if (TSharedPtr<IDatasmithMetaDataElement> MetaDataElement = SceneRef->GetMetaData(Element->GetDatasmithElement()))
		{
			return new FDatasmithFacadeMetaData(MetaDataElement.ToSharedRef());
		}
	}

	return nullptr;
}

void FDatasmithFacadeScene::RemoveMetaData(
	FDatasmithFacadeMetaData* InMetaDataPtr
)
{
	SceneRef->RemoveMetaData(InMetaDataPtr->GetDatasmithMetaDataElement());
}

//This function is a temporary workaround to make sure Materials and texture are not deleted from the scene.
//There won't be a need to reset the scene once all the Facade elements will stop generating DatasmithElement on the fly (and duplicating the data) during BuildElement()
void ResetBuiltFacadeElement(TSharedRef<IDatasmithScene>& SceneRef)
{
	TArray<TSharedPtr<IDatasmithMeshElement>> MeshesArray;
	TArray<TSharedPtr<IDatasmithBaseMaterialElement>> MaterialArray;
	TArray<TSharedPtr<IDatasmithTextureElement>> TextureArray;
	TArray<TSharedPtr<IDatasmithMetaDataElement>> MetaDataArray;
	TArray<TSharedPtr<IDatasmithActorElement>> ActorArray;
	TArray<TSharedPtr<IDatasmithLevelSequenceElement>> LevelSequences;
	TArray<TSharedPtr<IDatasmithLevelVariantSetsElement>> LevelVariantSets;

	const int32 MeshCount = SceneRef->GetMeshesCount();
	const int32 MaterialCount = SceneRef->GetMaterialsCount();
	const int32 TextureCount = SceneRef->GetTexturesCount();
	const int32 MetaDataCount = SceneRef->GetMetaDataCount();
	const int32 ActorCount = SceneRef->GetActorsCount();
	const int32 LevelSequencesCount = SceneRef->GetLevelSequencesCount();
	const int32 LevelVariantSetsCount = SceneRef->GetLevelVariantSetsCount();

	MeshesArray.Reserve( MeshCount );
	MaterialArray.Reserve( MaterialCount );
	TextureArray.Reserve( TextureCount );
	MetaDataArray.Reserve( MetaDataCount );
	ActorArray.Reserve( ActorCount );
	LevelSequences.Reserve( LevelSequencesCount );
	LevelVariantSets.Reserve( LevelVariantSetsCount );

	//Backup Meshes, Materials and Textures before reseting the scene in order to restore them.
	for ( int32 MeshIndex = 0; MeshIndex < MeshCount; ++MeshIndex )
	{
		MeshesArray.Add( SceneRef->GetMesh( MeshIndex ) );
	}

	for ( int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex )
	{
		MaterialArray.Add( SceneRef->GetMaterial( MaterialIndex ) );
	}

	for ( int32 TextureIndex = 0; TextureIndex < TextureCount; ++TextureIndex )
	{
		TextureArray.Add( SceneRef->GetTexture( TextureIndex ) );
	}

	for ( int32 MetaDataIndex = 0; MetaDataIndex < MetaDataCount; ++MetaDataIndex )
	{
		MetaDataArray.Add( SceneRef->GetMetaData( MetaDataIndex ) );
	}

	for ( int32 ActorIndex = 0; ActorIndex < ActorCount; ++ActorIndex )
	{
		ActorArray.Add( SceneRef->GetActor( ActorIndex ) );
	}

	for ( int32 LevelSequenceIndex = 0; LevelSequenceIndex < LevelSequencesCount; ++LevelSequenceIndex )
	{
		LevelSequences.Add( SceneRef->GetLevelSequence( LevelSequenceIndex ) );
	}

	for ( int32 LevelVariantSetIndex = 0; LevelVariantSetIndex < LevelVariantSetsCount; ++LevelVariantSetIndex )
	{
		LevelVariantSets.Add( SceneRef->GetLevelVariantSets( LevelVariantSetIndex ) );
	}

	SceneRef->Reset();

	for ( TSharedPtr<IDatasmithMeshElement>& CurrentMesh : MeshesArray )
	{
		SceneRef->AddMesh( CurrentMesh );
	}

	for ( TSharedPtr<IDatasmithBaseMaterialElement>& CurrentMaterial : MaterialArray )
	{
		SceneRef->AddMaterial( CurrentMaterial );
	}

	for ( TSharedPtr<IDatasmithTextureElement>& CurrentTexture: TextureArray )
	{
		SceneRef->AddTexture( CurrentTexture );
	}

	for ( TSharedPtr<IDatasmithMetaDataElement>& CurrentMetaData : MetaDataArray )
	{
		SceneRef->AddMetaData( CurrentMetaData );
	}

	for ( TSharedPtr<IDatasmithLevelSequenceElement>& CurrentLevelSequence : LevelSequences )
	{
		SceneRef->AddLevelSequence( CurrentLevelSequence.ToSharedRef() );
	}

	for ( TSharedPtr<IDatasmithLevelVariantSetsElement>& CurrentLevelVariantSet : LevelVariantSets )
	{
		SceneRef->AddLevelVariantSets( CurrentLevelVariantSet );
	}

	for ( TSharedPtr<IDatasmithActorElement>& CurrentActor : ActorArray )
	{
		SceneRef->AddActor( CurrentActor );
	}
}

void FDatasmithFacadeScene::CleanUp()
{
	FDatasmithSceneUtils::CleanUpScene(SceneRef, true);
}

void FDatasmithFacadeScene::PreExport()
{
	// Initialize the Datasmith exporter module.
	FDatasmithExporterManager::Initialize();

	// Start measuring the time taken to export the scene.
	SceneExporterRef->PreExport();
}

void FDatasmithFacadeScene::SetName(const TCHAR* InName)
{
	SceneExporterRef->SetName(InName);
	SceneRef->SetName(SceneExporterRef->GetName());
}

const TCHAR* FDatasmithFacadeScene::GetName() const
{
	return SceneExporterRef->GetName();
}

void FDatasmithFacadeScene::SetOutputPath(const TCHAR* InOutputPath)
{
	SceneExporterRef->SetOutputPath(InOutputPath);
	SceneRef->SetResourcePath(SceneExporterRef->GetOutputPath());
}

const TCHAR* FDatasmithFacadeScene::GetOutputPath() const
{
	return SceneExporterRef->GetOutputPath();
}

const TCHAR* FDatasmithFacadeScene::GetAssetsOutputPath() const
{
	return SceneExporterRef->GetAssetsOutputPath();
}

void FDatasmithFacadeScene::SetGeolocationLatitude(double Latitude)
{
	SceneRef->SetGeolocationLatitude(Latitude);
}

void FDatasmithFacadeScene::SetGeolocationLongitude(double Longitude)
{
	SceneRef->SetGeolocationLongitude(Longitude);
}

void FDatasmithFacadeScene::SetGeolocationElevation(double Elevation)
{
	SceneRef->SetGeolocationElevation(Elevation);
}

void FDatasmithFacadeScene::GetGeolocation(double& OutLatitude, double& OutLongitude, double& OutElevation) const
{
	FVector Geolocation = SceneRef->GetGeolocation();
	OutLatitude = Geolocation.X;
	OutLongitude = Geolocation.Y;
	OutElevation = Geolocation.Z;
}

void FDatasmithFacadeScene::Shutdown()
{
	FDatasmithExporterManager::Shutdown();
}

bool FDatasmithFacadeScene::ExportScene(
	const TCHAR* InOutputPath,
	bool bCleanupUnusedElements
)
{
	FString OutputPath = InOutputPath;

	// Set the name of the scene to export and let Datasmith sanitize it when required.
	FString SceneName = FPaths::GetBaseFilename(OutputPath);
	SetName(*SceneName);

	// Set the output folder where this scene will be exported.
	FString SceneFolder = FPaths::GetPath(OutputPath);
	SetOutputPath(*SceneFolder);

	return ExportScene(bCleanupUnusedElements);
}

bool FDatasmithFacadeScene::ExportScene(bool bCleanupUnusedElements)
{
	if ( FCString::Strlen(SceneExporterRef->GetName()) == 0
		|| FCString::Strlen(SceneExporterRef->GetOutputPath()) == 0)
	{
		return false;
	}

	// Export the Datasmith scene instance into its file.
	SceneExporterRef->Export(SceneRef, bCleanupUnusedElements);

	return true;
}

void FDatasmithFacadeScene::SerializeLevelSequences()
{
	FDatasmithAnimationSerializer AnimSerializer;
	int32 NumSequences = SceneRef->GetLevelSequencesCount();
	for (int32 SequenceIndex = 0; SequenceIndex < NumSequences; ++SequenceIndex)
	{
		const TSharedPtr<IDatasmithLevelSequenceElement>& LevelSequence = SceneRef->GetLevelSequence(SequenceIndex);
		if (LevelSequence.IsValid())
		{
			FString AnimFilePath = FPaths::Combine(SceneExporterRef->GetAssetsOutputPath(), LevelSequence->GetName()) + DATASMITH_ANIMATION_EXTENSION;

			if (AnimSerializer.Serialize(LevelSequence.ToSharedRef(), *AnimFilePath))
			{
				TUniquePtr<FArchive> AnimArchive(IFileManager::Get().CreateFileReader(*AnimFilePath));
				if (AnimArchive)
				{
					LevelSequence->SetFileHash(FMD5Hash::HashFileFromArchive(AnimArchive.Get()));
				}
				LevelSequence->SetFile(*AnimFilePath);
			}
		}
	}
}

TSharedRef<IDatasmithScene> FDatasmithFacadeScene::GetScene() const
{
	return SceneRef;
}

void FDatasmithFacadeScene::SetLabel(const TCHAR* InSceneLabel)
{
	SceneRef->SetLabel(InSceneLabel);
}

const TCHAR* FDatasmithFacadeScene::GetLabel() const
{
	return SceneRef->GetLabel();
}