// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeTexture.h"

#include "DatasmithFacadeScene.h"

#include "Misc/SecureHash.h"

FDatasmithFacadeTexture::FDatasmithFacadeTexture(const TCHAR* InElementName)
	: FDatasmithFacadeElement(FDatasmithSceneFactory::CreateTexture(InElementName))
{}

FDatasmithFacadeTexture::FDatasmithFacadeTexture(TSharedRef<IDatasmithTextureElement> InTextureElement)
	: FDatasmithFacadeElement(InTextureElement)
{}

const TCHAR* FDatasmithFacadeTexture::GetFile() const
{
	return GetDatasmithTextureElement()->GetFile();
}

void FDatasmithFacadeTexture::SetFile( const TCHAR* File )
{
	GetDatasmithTextureElement()->SetFile( File );
}

void FDatasmithFacadeTexture::SetData( const uint8* InData, uint32 InDataSize, ETextureFormat InFormat )
{
	GetDatasmithTextureElement()->SetData( InData, InDataSize, static_cast<EDatasmithTextureFormat>( InFormat ) );
}

const uint8* FDatasmithFacadeTexture::GetData( uint32& OutDataSize, ETextureFormat& OutFormat ) const
{
	EDatasmithTextureFormat OutInternalFormat;
	const uint8* Data = GetDatasmithTextureElement()->GetData( OutDataSize, OutInternalFormat );
	OutFormat = static_cast<ETextureFormat>( OutInternalFormat );

	return Data;
}

void FDatasmithFacadeTexture::GetFileHash( TCHAR OutBuffer[33], size_t BufferSize ) const
{
	FString HashString = LexToString( GetDatasmithTextureElement()->GetFileHash() );
	FCString::Strncpy( OutBuffer, *HashString, BufferSize );
}

void FDatasmithFacadeTexture::SetFileHash( const TCHAR* Hash )
{
	FMD5Hash Md5Hash;
	LexFromString( Md5Hash, Hash );

	GetDatasmithTextureElement()->SetFileHash( Md5Hash );
}

FDatasmithFacadeTexture::ETextureMode FDatasmithFacadeTexture::GetTextureMode() const
{
	return static_cast<ETextureMode>( GetDatasmithTextureElement()->GetTextureMode());
}

void FDatasmithFacadeTexture::SetTextureMode( ETextureMode Mode )
{
	GetDatasmithTextureElement()->SetTextureMode( static_cast<EDatasmithTextureMode>( Mode ) );
}

FDatasmithFacadeTexture::ETextureFilter FDatasmithFacadeTexture::GetTextureFilter() const
{
	return static_cast<ETextureFilter>( GetDatasmithTextureElement()->GetTextureFilter() );
}

void FDatasmithFacadeTexture::SetTextureFilter( ETextureFilter Filter )
{
	GetDatasmithTextureElement()->SetTextureFilter( static_cast<EDatasmithTextureFilter>( Filter ) );
}

FDatasmithFacadeTexture::ETextureAddress FDatasmithFacadeTexture::GetTextureAddressX() const
{
	return static_cast<ETextureAddress>( GetDatasmithTextureElement()->GetTextureAddressX() );
}

void FDatasmithFacadeTexture::SetTextureAddressX( ETextureAddress Mode )
{
	GetDatasmithTextureElement()->SetTextureAddressX( static_cast<EDatasmithTextureAddress>( Mode ) );
}

FDatasmithFacadeTexture::ETextureAddress FDatasmithFacadeTexture::GetTextureAddressY() const
{
	return static_cast<ETextureAddress>( GetDatasmithTextureElement()->GetTextureAddressY() );
}

void FDatasmithFacadeTexture::SetTextureAddressY( ETextureAddress Mode )
{
	GetDatasmithTextureElement()->SetTextureAddressY( static_cast<EDatasmithTextureAddress>( Mode ) );
}

bool FDatasmithFacadeTexture::GetAllowResize() const
{
	return GetDatasmithTextureElement()->GetAllowResize();
}

void FDatasmithFacadeTexture::SetAllowResize( bool bAllowResize )
{
	GetDatasmithTextureElement()->SetAllowResize( bAllowResize );
}

float FDatasmithFacadeTexture::GetRGBCurve() const
{
	return GetDatasmithTextureElement()->GetRGBCurve();
}

void FDatasmithFacadeTexture::SetRGBCurve( const float InRGBCurve )
{
	GetDatasmithTextureElement()->SetRGBCurve( InRGBCurve );
}

FDatasmithFacadeTexture::EColorSpace FDatasmithFacadeTexture::GetSRGB() const
{
	return static_cast<EColorSpace>( GetDatasmithTextureElement()->GetSRGB() );
}

void FDatasmithFacadeTexture::SetSRGB( EColorSpace Option )
{
	GetDatasmithTextureElement()->SetSRGB( static_cast<EDatasmithColorSpace>( Option ) );
}

TSharedRef<IDatasmithTextureElement> FDatasmithFacadeTexture::GetDatasmithTextureElement() const
{ 
	return StaticCastSharedRef<IDatasmithTextureElement>( InternalDatasmithElement );
}