// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeUEPbrMaterial.h"

#include "DatasmithFacadeKeyValueProperty.h"
#include "DatasmithFacadeScene.h"

#include "DatasmithUtils.h"
#include "Misc/Paths.h"

FDatasmithFacadeMaterialExpression* CreateFacadeExpression( IDatasmithMaterialExpression* MaterialExpression, const TSharedPtr<IDatasmithUEPbrMaterialElement>& ReferencedMaterial )
{
	if ( MaterialExpression )
	{
		switch ( MaterialExpression->GetExpressionType() )
		{
		case EDatasmithMaterialExpressionType::ConstantBool:
			return new FDatasmithFacadeMaterialExpressionBool( MaterialExpression, ReferencedMaterial );
		case EDatasmithMaterialExpressionType::ConstantColor:
			return new FDatasmithFacadeMaterialExpressionColor( MaterialExpression, ReferencedMaterial );
		case EDatasmithMaterialExpressionType::ConstantScalar:
			return new FDatasmithFacadeMaterialExpressionScalar( MaterialExpression, ReferencedMaterial );
		case EDatasmithMaterialExpressionType::FlattenNormal:
			return new FDatasmithFacadeMaterialExpressionFlattenNormal( MaterialExpression, ReferencedMaterial );
		case EDatasmithMaterialExpressionType::FunctionCall:
			return new FDatasmithFacadeMaterialExpressionFunctionCall( MaterialExpression, ReferencedMaterial );
		case EDatasmithMaterialExpressionType::Generic:
			return new FDatasmithFacadeMaterialExpressionGeneric( MaterialExpression, ReferencedMaterial );
		case EDatasmithMaterialExpressionType::Texture:
			return new FDatasmithFacadeMaterialExpressionTexture( MaterialExpression, ReferencedMaterial );
		case EDatasmithMaterialExpressionType::TextureCoordinate:
			return new FDatasmithFacadeMaterialExpressionTextureCoordinate( MaterialExpression, ReferencedMaterial );
		default:
			break;
		}
	}
	
	return nullptr;
}

const TCHAR* FDatasmithFacadeExpressionInput::GetName() const
{
	return InternalExpressionInput->GetName();
}

void FDatasmithFacadeExpressionInput::SetName(const TCHAR* InName)
{
	InternalExpressionInput->SetName(InName);
}

FDatasmithFacadeMaterialExpression* FDatasmithFacadeExpressionInput::GetNewFacadeExpression()
{
	return CreateFacadeExpression( InternalExpressionInput->GetExpression(), ReferencedMaterial );
}

void FDatasmithFacadeExpressionInput::SetExpression( FDatasmithFacadeMaterialExpression* InExpression )
{
	InternalExpressionInput->SetExpression( InExpression->GetMaterialExpression() );
}

int32 FDatasmithFacadeExpressionInput::GetOutputIndex() const
{
	return InternalExpressionInput->GetOutputIndex();
}

void FDatasmithFacadeExpressionInput::SetOutputIndex( int32 InOutputIndex )
{
	InternalExpressionInput->SetOutputIndex( InOutputIndex );
}

const TCHAR* FDatasmithFacadeMaterialExpression::GetName() const
{
	return InternalMaterialExpression->GetName();
}

void FDatasmithFacadeMaterialExpression::SetName( const TCHAR* InName )
{
	InternalMaterialExpression->SetName( InName );
}

EDatasmithFacadeMaterialExpressionType FDatasmithFacadeMaterialExpression::GetExpressionType() const
{
	return static_cast<EDatasmithFacadeMaterialExpressionType>(InternalMaterialExpression->GetExpressionType());
}

void FDatasmithFacadeMaterialExpression::ConnectExpression( FDatasmithFacadeExpressionInput& ExpressionInput )
{
	InternalMaterialExpression->ConnectExpression( ExpressionInput.GetExpressionInput() );
}

void FDatasmithFacadeMaterialExpression::ConnectExpression( FDatasmithFacadeExpressionInput& ExpressionInput, int32 OutputIndex )
{
	InternalMaterialExpression->ConnectExpression( ExpressionInput.GetExpressionInput(), OutputIndex );
}

int32 FDatasmithFacadeMaterialExpression::GetInputCount() const
{
	return InternalMaterialExpression->GetInputCount();
}

FDatasmithFacadeExpressionInput* FDatasmithFacadeMaterialExpression::GetNewFacadeInput( int32 Index )
{
	if ( IDatasmithExpressionInput* ExpressionInput = InternalMaterialExpression->GetInput( Index ) )
	{
		return new FDatasmithFacadeExpressionInput( ExpressionInput, ReferencedMaterial );
	}

	return nullptr;
}

int32 FDatasmithFacadeMaterialExpression::GetDefaultOutputIndex() const
{
	return InternalMaterialExpression->GetDefaultOutputIndex();
}

void FDatasmithFacadeMaterialExpression::SetDefaultOutputIndex( int32 OutputIndex )
{
	InternalMaterialExpression->SetDefaultOutputIndex( OutputIndex );
}

bool FDatasmithFacadeMaterialExpressionBool::GetBool() const
{
	return static_cast<IDatasmithMaterialExpressionBool*>( InternalMaterialExpression )->GetBool();
}

void FDatasmithFacadeMaterialExpressionBool::SetBool( bool InValue )
{
	static_cast<IDatasmithMaterialExpressionBool*>( InternalMaterialExpression )->GetBool() = InValue;
}

const TCHAR* FDatasmithFacadeMaterialExpressionBool::GetGroupName() const
{
	return static_cast<IDatasmithMaterialExpressionBool*>( InternalMaterialExpression )->GetGroupName();
}

void FDatasmithFacadeMaterialExpressionBool::SetGroupName( const TCHAR* InGroupName )
{
	static_cast<IDatasmithMaterialExpressionBool*>( InternalMaterialExpression )->SetGroupName( InGroupName );
}

void FDatasmithFacadeMaterialExpressionColor::GetsRGBColor( uint8& OutR, uint8& OutG, uint8& OutB, uint8& OutA ) const
{
	const FLinearColor& ExpressionColor = static_cast<IDatasmithMaterialExpressionColor*>( InternalMaterialExpression )->GetColor();
	FColor Color = ExpressionColor.ToFColor( /*bSRGB=*/true );
	OutR = Color.R;
	OutG = Color.G;
	OutB = Color.B;
	OutA = Color.A;
}

void FDatasmithFacadeMaterialExpressionColor::SetsRGBColor( uint8 R, uint8 G, uint8 B, uint8 A )
{
	//Passing a FColor to the FLinearColor constructor will do the proper color space conversion.
	static_cast<IDatasmithMaterialExpressionColor*>( InternalMaterialExpression )->GetColor() = FLinearColor( FColor( R, G, B, A ) );
}

void FDatasmithFacadeMaterialExpressionColor::GetColor( float& OutR, float& OutG, float& OutB, float& OutA ) const
{
	const FLinearColor& ExpressionColor = static_cast<IDatasmithMaterialExpressionColor*>( InternalMaterialExpression )->GetColor();
	OutR = ExpressionColor.R;
	OutG = ExpressionColor.G;
	OutB = ExpressionColor.B;
	OutA = ExpressionColor.A;
}

void FDatasmithFacadeMaterialExpressionColor::SetColor( float R, float G, float B, float A )
{
	static_cast<IDatasmithMaterialExpressionColor*>( InternalMaterialExpression )->GetColor() = FLinearColor( R, G, B, A );
}

const TCHAR* FDatasmithFacadeMaterialExpressionColor::GetGroupName() const
{
	return static_cast<IDatasmithMaterialExpressionColor*>( InternalMaterialExpression )->GetGroupName();
}

void FDatasmithFacadeMaterialExpressionColor::SetGroupName( const TCHAR* InGroupName )
{
	static_cast<IDatasmithMaterialExpressionColor*>( InternalMaterialExpression )->SetGroupName( InGroupName );
}

float FDatasmithFacadeMaterialExpressionScalar::GetScalar() const
{
	return static_cast<IDatasmithMaterialExpressionScalar*>( InternalMaterialExpression )->GetScalar();
}

void FDatasmithFacadeMaterialExpressionScalar::SetScalar( float InScalar )
{
	static_cast<IDatasmithMaterialExpressionScalar*>( InternalMaterialExpression )->GetScalar() = InScalar;
}

const TCHAR* FDatasmithFacadeMaterialExpressionScalar::GetGroupName() const
{
	return static_cast<IDatasmithMaterialExpressionScalar*>( InternalMaterialExpression )->GetGroupName();
}

void FDatasmithFacadeMaterialExpressionScalar::SetGroupName( const TCHAR* InGroupName )
{
	static_cast<IDatasmithMaterialExpressionScalar*>( InternalMaterialExpression )->SetGroupName( InGroupName );
}

const TCHAR* FDatasmithFacadeMaterialExpressionTexture::GetTexturePathName() const
{
	return static_cast<IDatasmithMaterialExpressionTexture*>( InternalMaterialExpression )->GetTexturePathName();
}

void FDatasmithFacadeMaterialExpressionTexture::SetTexturePathName( const TCHAR* InTexturePathName )
{
	static_cast<IDatasmithMaterialExpressionTexture*>( InternalMaterialExpression )->SetTexturePathName( InTexturePathName );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeMaterialExpressionTexture::GetInputCoordinate()
{
	return FDatasmithFacadeExpressionInput( &static_cast<IDatasmithMaterialExpressionTexture*>( InternalMaterialExpression )->GetInputCoordinate(), ReferencedMaterial );
}

const TCHAR* FDatasmithFacadeMaterialExpressionTexture::GetGroupName() const
{
	return static_cast<IDatasmithMaterialExpressionTexture*>( InternalMaterialExpression )->GetGroupName();
}

void FDatasmithFacadeMaterialExpressionTexture::SetGroupName( const TCHAR* InGroupName )
{
	static_cast<IDatasmithMaterialExpressionTexture*>( InternalMaterialExpression )->SetGroupName( InGroupName );
}

int32 FDatasmithFacadeMaterialExpressionTextureCoordinate::GetCoordinateIndex() const
{
	return static_cast<IDatasmithMaterialExpressionTextureCoordinate*>( InternalMaterialExpression )->GetCoordinateIndex();
}

void FDatasmithFacadeMaterialExpressionTextureCoordinate::SetCoordinateIndex( int32 InCoordinateIndex )
{
	static_cast<IDatasmithMaterialExpressionTextureCoordinate*>( InternalMaterialExpression )->SetCoordinateIndex( InCoordinateIndex );
}

float FDatasmithFacadeMaterialExpressionTextureCoordinate::GetUTiling() const
{
	return static_cast<IDatasmithMaterialExpressionTextureCoordinate*>( InternalMaterialExpression )->GetUTiling();
}

void FDatasmithFacadeMaterialExpressionTextureCoordinate::SetUTiling( float InUTiling )
{
	static_cast<IDatasmithMaterialExpressionTextureCoordinate*>( InternalMaterialExpression )->SetUTiling( InUTiling );
}

float FDatasmithFacadeMaterialExpressionTextureCoordinate::GetVTiling() const
{
	return static_cast<IDatasmithMaterialExpressionTextureCoordinate*>( InternalMaterialExpression )->GetVTiling();
}

void FDatasmithFacadeMaterialExpressionTextureCoordinate::SetVTiling( float InVTiling )
{
	static_cast<IDatasmithMaterialExpressionTextureCoordinate*>( InternalMaterialExpression )->SetVTiling( InVTiling );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeMaterialExpressionFlattenNormal::GetNormal() const
{
	return FDatasmithFacadeExpressionInput( &static_cast<IDatasmithMaterialExpressionFlattenNormal*>( InternalMaterialExpression )->GetNormal(), ReferencedMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeMaterialExpressionFlattenNormal::GetFlatness() const
{
	return FDatasmithFacadeExpressionInput( &static_cast<IDatasmithMaterialExpressionFlattenNormal*>( InternalMaterialExpression )->GetFlatness(), ReferencedMaterial );
}

void FDatasmithFacadeMaterialExpressionGeneric::SetExpressionName( const TCHAR* InExpressionName )
{
	static_cast<IDatasmithMaterialExpressionGeneric*>( InternalMaterialExpression )->SetExpressionName( InExpressionName );
}

const TCHAR* FDatasmithFacadeMaterialExpressionGeneric::GetExpressionName() const
{
	return static_cast<IDatasmithMaterialExpressionGeneric*>( InternalMaterialExpression )->GetExpressionName();
}

int32 FDatasmithFacadeMaterialExpressionGeneric::GetPropertiesCount() const
{
	return static_cast<IDatasmithMaterialExpressionGeneric*>( InternalMaterialExpression )->GetPropertiesCount();
}

void FDatasmithFacadeMaterialExpressionGeneric::AddProperty( const FDatasmithFacadeKeyValueProperty* InPropertyPtr )
{
	if ( InPropertyPtr )
	{
		static_cast<IDatasmithMaterialExpressionGeneric*>( InternalMaterialExpression )->AddProperty( InPropertyPtr->GetDatasmithKeyValueProperty() );
	}
}

FDatasmithFacadeKeyValueProperty* FDatasmithFacadeMaterialExpressionGeneric::GetNewProperty( int32 Index )
{
	if ( const TSharedPtr<IDatasmithKeyValueProperty>& Property = static_cast<IDatasmithMaterialExpressionGeneric*>( InternalMaterialExpression )->GetProperty( Index ) )
	{
		return new FDatasmithFacadeKeyValueProperty( Property.ToSharedRef() );
	}

	return nullptr;
}

void FDatasmithFacadeMaterialExpressionFunctionCall::SetFunctionPathName( const TCHAR* InFunctionPathName )
{
	static_cast<IDatasmithMaterialExpressionFunctionCall*>( InternalMaterialExpression )->SetFunctionPathName( InFunctionPathName );
}

const TCHAR* FDatasmithFacadeMaterialExpressionFunctionCall::GetFunctionPathName() const
{
	return static_cast<IDatasmithMaterialExpressionFunctionCall*>( InternalMaterialExpression )->GetFunctionPathName();
}

FDatasmithFacadeUEPbrMaterial::FDatasmithFacadeUEPbrMaterial( const TCHAR* InElementName )
	: FDatasmithFacadeBaseMaterial( FDatasmithSceneFactory::CreateUEPbrMaterial( InElementName ) )
{}

FDatasmithFacadeUEPbrMaterial::FDatasmithFacadeUEPbrMaterial( const TSharedRef<IDatasmithUEPbrMaterialElement>& InMaterialRef )
	: FDatasmithFacadeBaseMaterial( InMaterialRef )
{}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetBaseColor() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetBaseColor(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetMetallic() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetMetallic(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetSpecular() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetSpecular(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetRoughness() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetRoughness(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetEmissiveColor() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetEmissiveColor(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetOpacity() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetOpacity(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetNormal() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetNormal(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetRefraction() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetRefraction(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetAmbientOcclusion() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetAmbientOcclusion(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetClearCoat() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetClearCoat(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetClearCoatRoughness() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetClearCoatRoughness(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetWorldPositionOffset() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetWorldPositionOffset(), UEPbrMaterial );
}

FDatasmithFacadeExpressionInput FDatasmithFacadeUEPbrMaterial::GetMaterialAttributes() const
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return FDatasmithFacadeExpressionInput( &UEPbrMaterial->GetMaterialAttributes(), UEPbrMaterial );
}

int FDatasmithFacadeUEPbrMaterial::GetBlendMode() const
{
	return GetDatasmithUEPbrMaterialElement()->GetBlendMode();
}

void FDatasmithFacadeUEPbrMaterial::SetBlendMode( int bInBlendMode )
{
	GetDatasmithUEPbrMaterialElement()->SetBlendMode( bInBlendMode );
}

bool FDatasmithFacadeUEPbrMaterial::GetTwoSided() const
{
	return GetDatasmithUEPbrMaterialElement()->GetTwoSided();
}

void FDatasmithFacadeUEPbrMaterial::SetTwoSided( bool bTwoSided )
{
	GetDatasmithUEPbrMaterialElement()->SetTwoSided( bTwoSided );
}

bool FDatasmithFacadeUEPbrMaterial::GetUseMaterialAttributes() const
{
	return GetDatasmithUEPbrMaterialElement()->GetUseMaterialAttributes();
}

void FDatasmithFacadeUEPbrMaterial::SetUseMaterialAttributes( bool bInUseMaterialAttributes )
{
	GetDatasmithUEPbrMaterialElement()->SetUseMaterialAttributes( bInUseMaterialAttributes );
}

bool FDatasmithFacadeUEPbrMaterial::GetMaterialFunctionOnly() const
{
	return GetDatasmithUEPbrMaterialElement()->GetMaterialFunctionOnly();
}

void FDatasmithFacadeUEPbrMaterial::SetMaterialFunctionOnly( bool bInMaterialFunctionOnly )
{
	GetDatasmithUEPbrMaterialElement()->SetMaterialFunctionOnly( bInMaterialFunctionOnly );
}

float FDatasmithFacadeUEPbrMaterial::GetOpacityMaskClipValue() const
{
	return GetDatasmithUEPbrMaterialElement()->GetOpacityMaskClipValue();
}

void FDatasmithFacadeUEPbrMaterial::SetOpacityMaskClipValue( float InClipValue )
{
	GetDatasmithUEPbrMaterialElement()->SetOpacityMaskClipValue( InClipValue );
}

int32 FDatasmithFacadeUEPbrMaterial::GetExpressionsCount() const
{
	return GetDatasmithUEPbrMaterialElement()->GetExpressionsCount();
}

FDatasmithFacadeMaterialExpression* FDatasmithFacadeUEPbrMaterial::GetNewFacadeExpression( int32 Index )
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> UEPbrMaterial = GetDatasmithUEPbrMaterialElement();
	return CreateFacadeExpression( UEPbrMaterial->GetExpression( Index ), UEPbrMaterial );
}

int32 FDatasmithFacadeUEPbrMaterial::GetExpressionIndex( const FDatasmithFacadeMaterialExpression& Expression ) const
{
	return GetDatasmithUEPbrMaterialElement()->GetExpressionIndex( Expression.GetMaterialExpression() );
}

IDatasmithMaterialExpression* FDatasmithFacadeUEPbrMaterial::AddMaterialExpression( const EDatasmithFacadeMaterialExpressionType ExpressionType )
{
	return GetDatasmithUEPbrMaterialElement()->AddMaterialExpression( static_cast<EDatasmithMaterialExpressionType>( ExpressionType ) );
}

void FDatasmithFacadeUEPbrMaterial::SetParentLabel( const TCHAR* InParentLabel )
{
	GetDatasmithUEPbrMaterialElement()->SetParentLabel( InParentLabel );
}

const TCHAR* FDatasmithFacadeUEPbrMaterial::GetParentLabel() const
{
	return GetDatasmithUEPbrMaterialElement()->GetParentLabel();
}

TSharedRef<IDatasmithUEPbrMaterialElement> FDatasmithFacadeUEPbrMaterial::GetDatasmithUEPbrMaterialElement() const
{
	return StaticCastSharedRef<IDatasmithUEPbrMaterialElement>( InternalDatasmithElement );
}