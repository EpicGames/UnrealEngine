// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeUtils.h"
#include "DatasmithUtils.h"

FDatasmithFacadeUniqueNameProvider::FDatasmithFacadeUniqueNameProvider()
	:InternalNameProvider(MakeUnique<FDatasmithUniqueNameProvider>())
{}


const TCHAR* FDatasmithFacadeUniqueNameProvider::GenerateUniqueName(const TCHAR* BaseName)
{
	CachedGeneratedName = InternalNameProvider->GenerateUniqueName(BaseName);
	return *CachedGeneratedName;
}

void FDatasmithFacadeUniqueNameProvider::Reserve( int32 NumberOfName )
{
	InternalNameProvider->Reserve(NumberOfName);
}

void FDatasmithFacadeUniqueNameProvider::AddExistingName(const TCHAR* Name)
{
	InternalNameProvider->AddExistingName(Name);
}

void FDatasmithFacadeUniqueNameProvider::RemoveExistingName(const TCHAR* Name)
{
	InternalNameProvider->RemoveExistingName(Name);
}
