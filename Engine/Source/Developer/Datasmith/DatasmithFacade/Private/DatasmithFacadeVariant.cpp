// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeVariant.h"
#include "DatasmithFacadeMaterial.h"
#include "Math/Transform.h"

FDatasmithFacadePropertyCapture::FDatasmithFacadePropertyCapture()
: FDatasmithFacadeElement(FDatasmithSceneFactory::CreatePropertyCapture())
{
}

FDatasmithFacadePropertyCapture::FDatasmithFacadePropertyCapture(
	const TSharedRef<IDatasmithBasePropertyCaptureElement>& InInternalPropertyCapture
) :
	FDatasmithFacadeElement( InInternalPropertyCapture )
{
}

void FDatasmithFacadePropertyCapture::SetPropertyPath(const TCHAR* Path)
{
	GetDatasmithPropertyCapture()->SetPropertyPath(Path);
}

const TCHAR* FDatasmithFacadePropertyCapture::GetPropertyPath()
{
	return *GetDatasmithPropertyCapture()->GetPropertyPath();
}

EDatasmithFacadePropertyCategory FDatasmithFacadePropertyCapture::GetCategory() const
{
	return static_cast<EDatasmithFacadePropertyCategory>(GetDatasmithPropertyCapture()->GetCategory());
}

TSharedRef<IDatasmithBasePropertyCaptureElement> FDatasmithFacadePropertyCapture::GetDatasmithPropertyCapture() const
{
	return StaticCastSharedRef<IDatasmithBasePropertyCaptureElement>( InternalDatasmithElement );
}

FDatasmithFacadeActorBinding::FDatasmithFacadeActorBinding(
	FDatasmithFacadeActor* InActorPtr
) :
	FDatasmithFacadeElement(FDatasmithSceneFactory::CreateActorBinding())
{
	GetDatasmithActorBinding()->SetActor(InActorPtr->GetDatasmithActorElement());
}

FDatasmithFacadeActorBinding::FDatasmithFacadeActorBinding(
	const TSharedRef<IDatasmithActorBindingElement>& InInternalActorBinding
) :
	FDatasmithFacadeElement( InInternalActorBinding )
{
}

TSharedRef<IDatasmithActorBindingElement> FDatasmithFacadeActorBinding::GetDatasmithActorBinding() const
{
	return StaticCastSharedRef<IDatasmithActorBindingElement>( InternalDatasmithElement );
}

void FDatasmithFacadeActorBinding::AddRelativeLocationCapture(
	double X,
	double Y,
	double Z
)
{
	FVector Location(X, Y, Z);
	TSharedRef<IDatasmithPropertyCaptureElement> LocationProperty = FDatasmithSceneFactory::CreatePropertyCapture();
	LocationProperty->SetCategory(EDatasmithPropertyCategory::RelativeLocation);
	LocationProperty->SetRecordedData((uint8*)&Location, sizeof(FVector));
	GetDatasmithActorBinding()->AddPropertyCapture(LocationProperty);
}

void FDatasmithFacadeActorBinding::AddRelativeRotationCapture(
	double Pitch,
	double Yaw,
	double Roll
)
{
	FRotator Rotation(Pitch, Yaw, Roll);
	TSharedRef<IDatasmithPropertyCaptureElement> RotationProperty = FDatasmithSceneFactory::CreatePropertyCapture();
	RotationProperty->SetCategory(EDatasmithPropertyCategory::RelativeRotation);
	RotationProperty->SetRecordedData((uint8*)&Rotation, sizeof(FRotator));
	GetDatasmithActorBinding()->AddPropertyCapture(RotationProperty);
}

void FDatasmithFacadeActorBinding::AddRelativeRotationCapture(
	double X,
	double Y,
	double Z,
	double W
)
{
	FRotator Rotation(FQuat(X, Y, Z, W));
	TSharedRef<IDatasmithPropertyCaptureElement> RotationProperty = FDatasmithSceneFactory::CreatePropertyCapture();
	RotationProperty->SetCategory(EDatasmithPropertyCategory::RelativeRotation);
	RotationProperty->SetRecordedData((uint8*)&Rotation, sizeof(FRotator));
	GetDatasmithActorBinding()->AddPropertyCapture(RotationProperty);
}

void FDatasmithFacadeActorBinding::AddRelativeScaleCapture(
	double X,
	double Y,
	double Z
)
{
	FVector Scale(X, Y, Z);
	TSharedRef<IDatasmithPropertyCaptureElement> ScaleProperty = FDatasmithSceneFactory::CreatePropertyCapture();
	ScaleProperty->SetCategory(EDatasmithPropertyCategory::RelativeScale3D);
	ScaleProperty->SetRecordedData((uint8*)&Scale, sizeof(FVector));
	GetDatasmithActorBinding()->AddPropertyCapture(ScaleProperty);
}

void FDatasmithFacadeActorBinding::AddRelativeTransformCapture(
	const double InMatrix[16],
	bool bRowMajor
)
{
	TSharedPtr<IDatasmithActorElement> Actor = GetDatasmithActorBinding()->GetActor();
	FDatasmithFacadeActor FacadeActor(Actor.ToSharedRef()); //todo: probably could reuse existing actor reference, or make ConvertTransform static

	// Decompose matrix to FTransform with use of FDatasmithFacadeActor::ConvertTransform()
	FTransform Transform = FacadeActor.ConvertTransform(InMatrix, bRowMajor);

	//todo: can inline these functions
	FVector Scale = Transform.GetScale3D();
	FVector Location = Transform.GetTranslation();
	FQuat Rotation = Transform.GetRotation();

	AddRelativeScaleCapture(Scale.X, Scale.Y, Scale.Z);
	AddRelativeLocationCapture(Location.X, Location.Y, Location.Z);
	AddRelativeRotationCapture(Rotation.X, Rotation.Y, Rotation.Z, Rotation.W);
}

void FDatasmithFacadeActorBinding::AddVisibilityCapture(
	bool bInVisibility
)
{
	TSharedRef<IDatasmithPropertyCaptureElement> VisibilityProperty = FDatasmithSceneFactory::CreatePropertyCapture();
	VisibilityProperty->SetCategory(EDatasmithPropertyCategory::Visibility);
	VisibilityProperty->SetRecordedData((uint8*)&bInVisibility, sizeof(bool));
	GetDatasmithActorBinding()->AddPropertyCapture(VisibilityProperty);
}

void FDatasmithFacadeActorBinding::AddMaterialCapture(
	FDatasmithFacadeBaseMaterial* InMaterialPtr
)
{
	TSharedRef<IDatasmithObjectPropertyCaptureElement> MaterialProperty = FDatasmithSceneFactory::CreateObjectPropertyCapture();
	MaterialProperty->SetCategory(EDatasmithPropertyCategory::Material);
	MaterialProperty->SetRecordedObject(InMaterialPtr->GetDatasmithBaseMaterial());
	GetDatasmithActorBinding()->AddPropertyCapture(MaterialProperty);
}

void FDatasmithFacadeActorBinding::AddPropertyCapture(
	FDatasmithFacadePropertyCapture* InPropertyCapturePtr
)
{
	if (InPropertyCapturePtr)
	{
		GetDatasmithActorBinding()->AddPropertyCapture(InPropertyCapturePtr->GetDatasmithPropertyCapture());
	}
}

int32 FDatasmithFacadeActorBinding::GetPropertyCapturesCount() const
{
	return GetDatasmithActorBinding()->GetPropertyCapturesCount();
}

FDatasmithFacadePropertyCapture* FDatasmithFacadeActorBinding::GetNewPropertyCapture(
	int32 PropertyCaptureIndex
)
{
	if (TSharedPtr<IDatasmithBasePropertyCaptureElement> PropertyCaptureElement = GetDatasmithActorBinding()->GetPropertyCapture(PropertyCaptureIndex))
	{
		return new FDatasmithFacadePropertyCapture(PropertyCaptureElement.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeActorBinding::RemovePropertyCapture(
	FDatasmithFacadePropertyCapture* InPropertyCapturePtr
)
{
	GetDatasmithActorBinding()->RemovePropertyCapture(InPropertyCapturePtr->GetDatasmithPropertyCapture());
}

FDatasmithFacadeVariant::FDatasmithFacadeVariant(
	const TCHAR* InElementName
) :
	FDatasmithFacadeElement(FDatasmithSceneFactory::CreateVariant(InElementName))
{
}

FDatasmithFacadeVariant::FDatasmithFacadeVariant(
	const TSharedRef<IDatasmithVariantElement>& InInternalVariant
) :
	FDatasmithFacadeElement( InInternalVariant )
{
}

TSharedRef<IDatasmithVariantElement> FDatasmithFacadeVariant::GetDatasmithVariant() const
{
	return StaticCastSharedRef<IDatasmithVariantElement>( InternalDatasmithElement );
}

void FDatasmithFacadeVariant::AddActorBinding(
	FDatasmithFacadeActorBinding* InActorBindingPtr
)
{
	if (InActorBindingPtr)
	{
		GetDatasmithVariant()->AddActorBinding(InActorBindingPtr->GetDatasmithActorBinding());
	}
}

int32 FDatasmithFacadeVariant::GetActorBindingsCount() const
{
	return GetDatasmithVariant()->GetActorBindingsCount();
}

FDatasmithFacadeActorBinding* FDatasmithFacadeVariant::GetNewActorBinding(
	int32 ActorBindingIndex
)
{
	if (TSharedPtr<IDatasmithActorBindingElement> ActorBindingElement = GetDatasmithVariant()->GetActorBinding(ActorBindingIndex))
	{
		return new FDatasmithFacadeActorBinding(ActorBindingElement.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeVariant::RemoveActorBinding(
	FDatasmithFacadeActorBinding* InActorBindingPtr
)
{
	GetDatasmithVariant()->RemoveActorBinding(InActorBindingPtr->GetDatasmithActorBinding());
}

FDatasmithFacadeVariantSet::FDatasmithFacadeVariantSet(
	const TCHAR* InElementName
) :
	FDatasmithFacadeElement(FDatasmithSceneFactory::CreateVariantSet(InElementName))
{
}

FDatasmithFacadeVariantSet::FDatasmithFacadeVariantSet(
	const TSharedRef<IDatasmithVariantSetElement>& InInternalVariantSet
) :
	FDatasmithFacadeElement( InInternalVariantSet )
{
}

TSharedRef<IDatasmithVariantSetElement> FDatasmithFacadeVariantSet::GetDatasmithVariantSet() const
{
	return StaticCastSharedRef<IDatasmithVariantSetElement>( InternalDatasmithElement );
}

void FDatasmithFacadeVariantSet::AddVariant(
	FDatasmithFacadeVariant* InVariantPtr
)
{
	if (InVariantPtr)
	{
		GetDatasmithVariantSet()->AddVariant(InVariantPtr->GetDatasmithVariant());
	}
}

int32 FDatasmithFacadeVariantSet::GetVariantsCount() const
{
	return GetDatasmithVariantSet()->GetVariantsCount();
}

FDatasmithFacadeVariant* FDatasmithFacadeVariantSet::GetNewVariant(
	int32 VariantIndex
)
{
	if (TSharedPtr<IDatasmithVariantElement> VariantElement = GetDatasmithVariantSet()->GetVariant(VariantIndex))
	{
		return new FDatasmithFacadeVariant(VariantElement.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeVariantSet::RemoveVariant(
	FDatasmithFacadeVariant* InVariantPtr
)
{
	GetDatasmithVariantSet()->RemoveVariant(InVariantPtr->GetDatasmithVariant());
}

FDatasmithFacadeLevelVariantSets::FDatasmithFacadeLevelVariantSets(
	const TCHAR* InElementName
) :
	FDatasmithFacadeElement(FDatasmithSceneFactory::CreateLevelVariantSets(InElementName))
{
}

FDatasmithFacadeLevelVariantSets::FDatasmithFacadeLevelVariantSets(
	const TSharedRef<IDatasmithLevelVariantSetsElement>& InInternalLevelVariantsSet
) :
	FDatasmithFacadeElement( InInternalLevelVariantsSet )
{
}

TSharedRef<IDatasmithLevelVariantSetsElement> FDatasmithFacadeLevelVariantSets::GetDatasmithLevelVariantSets() const
{
	return StaticCastSharedRef<IDatasmithLevelVariantSetsElement>( InternalDatasmithElement );
}

void FDatasmithFacadeLevelVariantSets::AddVariantSet(
	FDatasmithFacadeVariantSet* InVariantSetPtr
)
{
	if (InVariantSetPtr)
	{
		GetDatasmithLevelVariantSets()->AddVariantSet(InVariantSetPtr->GetDatasmithVariantSet());
	}
}

int32 FDatasmithFacadeLevelVariantSets::GetVariantSetsCount() const
{
	return GetDatasmithLevelVariantSets()->GetVariantSetsCount();
}

FDatasmithFacadeVariantSet* FDatasmithFacadeLevelVariantSets::GetNewVariantSet(
	int32 VariantSetIndex
)
{
	if (TSharedPtr<IDatasmithVariantSetElement> VariantSetElement = GetDatasmithLevelVariantSets()->GetVariantSet(VariantSetIndex))
	{
		return new FDatasmithFacadeVariantSet(VariantSetElement.ToSharedRef());
	}

	return nullptr;
}

void FDatasmithFacadeLevelVariantSets::RemoveVariantSet(
	FDatasmithFacadeVariantSet* InVariantSetPtr
)
{
	GetDatasmithLevelVariantSets()->RemoveVariantSet(InVariantSetPtr->GetDatasmithVariantSet());
}
