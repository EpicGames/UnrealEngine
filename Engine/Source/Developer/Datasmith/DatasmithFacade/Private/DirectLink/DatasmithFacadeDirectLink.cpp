// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLink/DatasmithFacadeDirectLink.h"
#include "DatasmithFacadeLog.h"
#include "DatasmithFacadeScene.h"
#include "DirectLink/DatasmithFacadeEndpointObserver.h"
#include "DirectLink/DatasmithFacadeEndpointObserverImpl.h"

#include "DatasmithExporterManager.h"
#include "DirectLinkEndpoint.h"
#include "Modules/ModuleManager.h"
#include "Misc/CommandLine.h"

#include "DatasmithDirectLink.h"

bool FDatasmithFacadeDirectLink::Init()
{
	return Init(false, nullptr);
}

bool FDatasmithFacadeDirectLink::Init(bool bUseDatasmithExporterUI, const TCHAR* RemoteEngineDirPath)
{
	FDatasmithExporterManager::FInitOptions Options;
	Options.bEnableMessaging = true; // DirectLink requires the Messaging service.
	Options.bSuppressLogs = false;   // Log are useful, don't suppress them
	Options.bUseDatasmithExporterUI = bUseDatasmithExporterUI;
	Options.RemoteEngineDirPath = RemoteEngineDirPath;

	// #ue_directlink_cleanup it's not our role to init/deinit exporter manager here. See also Shutdown()
	if (!FDatasmithExporterManager::Initialize(Options))
	{
		UE_LOGF(LogDatasmithFacade, Error, "Fail to initialize FDatasmithExporterManager");
		return false;
	}

	if (int32 ErrorCode = FDatasmithDirectLink::ValidateCommunicationSetup())
	{
		UE_LOGF(LogDatasmithFacade, Error, "Communication setup issue: ErrorCode=%d", ErrorCode);
		return false;
	}

	UE_LOGF(LogDatasmithFacade, Display, "FDatasmithFacadeDirectLink Init OK");
	return true;
}

bool FDatasmithFacadeDirectLink::Shutdown()
{
	FDatasmithDirectLink::Shutdown();
	FDatasmithExporterManager::Shutdown();
	return true;
}

bool FDatasmithFacadeDirectLink::InitializeForScene(FDatasmithFacadeScene* FacadeScene)
{
	if (FacadeScene)
	{
		TSharedRef<IDatasmithScene> Scene = FacadeScene->GetScene();
		return Impl.InitializeForScene(Scene);
	}
	return false;
}

bool FDatasmithFacadeDirectLink::UpdateScene(FDatasmithFacadeScene* FacadeScene)
{
	if (FacadeScene)
	{
		TSharedRef<IDatasmithScene> Scene = FacadeScene->GetScene();
		return Impl.UpdateScene(Scene);
	}
	return false;
}

void FDatasmithFacadeDirectLink::CloseCurrentSource()
{
	Impl.CloseCurrentSource();
}

void FDatasmithFacadeDirectLink::AddEndpointObserver(FDatasmithFacadeEndpointObserver* Observer)
{
	if (Observer)
	{
		Impl.GetEnpoint()->AddEndpointObserver(&*Observer->GetObserver());
	}
}

void FDatasmithFacadeDirectLink::RemoveEndpointObserver(FDatasmithFacadeEndpointObserver* Observer)
{
	if (Observer)
	{
		Impl.GetEnpoint()->RemoveEndpointObserver(&*Observer->GetObserver());
	}
}