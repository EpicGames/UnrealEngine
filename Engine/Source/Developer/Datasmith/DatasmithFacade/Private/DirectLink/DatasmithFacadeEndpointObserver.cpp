// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLink/DatasmithFacadeEndpointObserver.h"
#include "DirectLink/DatasmithFacadeEndpointObserverImpl.h"


FDatasmithFacadeRawInfo::FDatasmithFacadeDataPointId* FDatasmithFacadeRawInfo::FDatasmithFacadeEndpointInfo::GetNewDestination(int32 Index) const
{ 
	if (EndpointInfo.Destinations.IsValidIndex(Index))
	{
		return new FDatasmithFacadeDataPointId(EndpointInfo.Destinations[Index]);
	}
	return nullptr;
}

FDatasmithFacadeRawInfo::FDatasmithFacadeDataPointId* FDatasmithFacadeRawInfo::FDatasmithFacadeEndpointInfo::GetNewSource(int32 Index) const
{
	if (EndpointInfo.Sources.IsValidIndex(Index))
	{
		return new FDatasmithFacadeDataPointId(EndpointInfo.Sources[Index]);
	}
	return nullptr;
}

FDatasmithFacadeRawInfo::FDatasmithFacadeEndpointInfo* FDatasmithFacadeRawInfo::GetNewEndpointInfo(const FMessageAddress* MessageAddress) const
{
	const DirectLink::FRawInfo::FEndpointInfo* EndPointInfo = MessageAddress ? RawInfo.EndpointsInfo.Find(*MessageAddress) : nullptr;
	
	if (EndPointInfo)
	{
		return new FDatasmithFacadeEndpointInfo(*EndPointInfo);
	}
	return nullptr;
}

FDatasmithFacadeRawInfo::FDatasmithFacadeDataPointInfo* FDatasmithFacadeRawInfo::GetNewDataPointsInfo(const FGuid* Id) const
{
	const DirectLink::FRawInfo::FDataPointInfo* DataPointInfo = Id ? RawInfo.DataPointsInfo.Find(*Id) : nullptr;

	if (DataPointInfo)
	{
		return new FDatasmithFacadeDataPointInfo(*DataPointInfo);
	}
	return nullptr;
}

FDatasmithFacadeRawInfo::FDatasmithFacadeStreamInfo* FDatasmithFacadeRawInfo::GetNewStreamInfo(int32 Index) const
{
	if (RawInfo.StreamsInfo.IsValidIndex(Index))
	{
		return new FDatasmithFacadeStreamInfo(RawInfo.StreamsInfo[Index]);
	}
	return nullptr;
}

FDatasmithFacadeEndpointObserver::FDatasmithFacadeEndpointObserver()
	: ObserverImpl(MakeShared<FDatasmithFacadeEndpointObserverImpl>())
{}

void FDatasmithFacadeEndpointObserver::RegisterOnStateChangedDelegateInternal(FDatasmithFacadeEndpointObserver::OnStateChangedDelegate InOnStateChangedDelegate)
{
	ObserverImpl->RegisterOnStateChangedDelegate(InOnStateChangedDelegate);
}

void FDatasmithFacadeEndpointObserver::UnregisterOnStateChangedDelegateInternal(FDatasmithFacadeEndpointObserver::OnStateChangedDelegate InOnStateChangedDelegate)
{
	ObserverImpl->UnregisterOnStateChangedDelegate(InOnStateChangedDelegate);
}
