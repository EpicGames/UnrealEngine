// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkEndpoint.h"
#include "DirectLink/DatasmithFacadeEndpointObserver.h"


class FDatasmithFacadeEndpointObserverImpl : public DirectLink::IEndpointObserver
{
public:
	virtual void OnStateChanged(const DirectLink::FRawInfo& RawInfo) override
	{
		if (OnStateChangedPtr)
		{
			// The new pointer will be owned by the C# wrapper.
			OnStateChangedPtr(new FDatasmithFacadeRawInfo(RawInfo));
		}
	}

	void RegisterOnStateChangedDelegate(FDatasmithFacadeEndpointObserver::OnStateChangedDelegate InOnStateChangedDelegate)
	{
		if (ensure(!OnStateChangedPtr))
		{
			OnStateChangedPtr = InOnStateChangedDelegate;
		}
	}

	void UnregisterOnStateChangedDelegate(FDatasmithFacadeEndpointObserver::OnStateChangedDelegate InOnStateChangedDelegate)
	{
		if (ensure(OnStateChangedPtr == InOnStateChangedDelegate))
		{
			OnStateChangedPtr = nullptr;
		}
	}

private:
	FDatasmithFacadeEndpointObserver::OnStateChangedDelegate OnStateChangedPtr;
};