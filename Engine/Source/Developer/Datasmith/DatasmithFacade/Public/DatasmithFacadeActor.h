// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeElement.h"

class IDatasmithActorElement;
class IDatasmithMetaDataElement;

class DATASMITHFACADE_API FDatasmithFacadeActor :
	public FDatasmithFacadeElement
{
public:

	enum class EActorType
	{
		DirectionalLight,
		AreaLight,
		EnvironmentLight,
		LightmassPortal,
		PointLight,
		SpotLight,
		StaticMeshActor,
		Camera,
		Actor,
		Decal,
		Unsupported,
	};

	FDatasmithFacadeActor(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadeActor() {}

	void SetWorldTransform(
		const double InWorldMatrix[16],
		bool bRowMajor = false
	);

	void SetWorldTransform(
		const float InWorldMatrix[16],
		bool bRowMajor = false
	)
	{
		double InMatrixD[16];
		for(int I = 0; I < 16; ++I)
		{
			InMatrixD[I] = InWorldMatrix[I];
		}
		SetWorldTransform(InMatrixD, bRowMajor);
	}

	void SetScale(
		double X,
		double Y,
		double Z
	);

	void GetScale(
		double& OutX,
		double& OutY,
		double& OutZ
	) const;

	void SetRotation(
		double Pitch,
		double Yaw,
		double Roll
	);

	void GetRotation(
		double& OutPitch,
		double& OutYaw,
		double& OutRoll
	) const;

	void SetRotation(
		double X,
		double Y,
		double Z,
		double W
	);

	void GetRotation(
		double& OutX,
		double& OutY,
		double& OutZ,
		double& OutW
	) const;

	void SetTranslation(
		double X,
		double Y,
		double Z
	);

	void GetTranslation(
		double& OutX,
		double& OutY,
		double& OutZ
	) const;

	// Set the layer of the Datasmith actor.
	void SetLayer(
		const TCHAR* InLayerName
	);

	// Get the layer of the Datasmith actor.
	const TCHAR* GetLayer() const;

	// Add a new tag to the Datasmith actor.
	void AddTag(
		const TCHAR* InTag
	);

	// Remove all Tags on the Actor element
	void ResetTags();

	// Get the number of tags attached to an Actor element
	int32 GetTagsCount() const;

	// Get the 'TagIndex'th tag of an Actor element
	const TCHAR* GetTag(
		int32 TagIndex
	) const;

	// Get whether or not the Datasmith actor is a component when used in a hierarchy.
	bool IsComponent() const;

	// Set whether or not the Datasmith actor is a component when used in a hierarchy.
	void SetIsComponent(
		bool bInIsComponent
	);

	// Add a new child to the Datasmith actor.
	void AddChild(
		FDatasmithFacadeActor* InChildActorPtr // Datasmith child actor
	);

	// Get the number of children on this actor
	int32 GetChildrenCount() const;

	/**
	 *	Returns a new FDatasmithFacadeActor pointing to the InIndex-th child of the mesh actor
	 *	If there is no child at the given index, returned value is nullptr.
	 *	The caller is responsible of deleting the returned object pointer.
	 */
	FDatasmithFacadeActor* GetNewChild(
		int32 InIndex
	);

	void RemoveChild(
		FDatasmithFacadeActor* InChild
	);

	/**
	 *	Returns a new FDatasmithFacadeActor pointing to the parent Actor of the actor
	 *	If there is no parent of if actor is directly under the scene root, the returned value is nullptr.
	 *	The caller is responsible of deleting the returned object pointer.
	 */
	FDatasmithFacadeActor* GetNewParentActor() const
	{
		return GetNewFacadeActorFromSharedPtr( GetDatasmithActorElement()->GetParentActor() );
	}

	// Get a mesh actor's visibility
	void SetVisibility(
		bool bInVisibility
	);

	// Set a mesh actor's visibility
	bool GetVisibility() const;

	void SetCastShadow(
		bool bInCastShadow
	);

	bool GetCastShadow() const;

	EActorType GetActorType() const;

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadeActor(
		const TSharedRef<IDatasmithActorElement>& InInternalActor
	);

	static EActorType GetActorType(
		const TSharedPtr<const IDatasmithActorElement>& InActor
	);

	static FDatasmithFacadeActor* GetNewFacadeActorFromSharedPtr(
		const TSharedPtr<IDatasmithActorElement>& InActor
	);

	// Convert a source matrix into a Datasmith actor transform.
	FTransform ConvertTransform(
		const double InSourceMatrix[16],
		bool bRowMajor
	) const;

	TSharedRef<IDatasmithActorElement> GetDatasmithActorElement() const;
};
