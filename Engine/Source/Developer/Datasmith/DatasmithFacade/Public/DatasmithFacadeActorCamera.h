// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeActor.h"


class DATASMITHFACADE_API FDatasmithFacadeActorCamera :
	public FDatasmithFacadeActor
{
public:

	FDatasmithFacadeActorCamera(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadeActorCamera() {}

	// Set the world position of the Datasmith camera actor.
	void SetCameraPosition(
		double InX, // camera position on the X axis
		double InY, // camera position on the Y axis
		double InZ  // camera position on the Z axis
	);

	// Set the world rotation of the Datasmith camera actor with the camera world forward and up directions.
	void SetCameraRotation(
		double InForwardX, // camera forward direction on the X axis
		double InForwardY, // camera forward direction on the Y axis
		double InForwardZ, // camera forward direction on the Z axis
		double InUpX,      // camera up direction on the X axis
		double InUpY,      // camera up direction on the Y axis
		double InUpZ       // camera up direction on the Z axis
	);

	// Set the sensor width of the Datasmith camera.
	void SetSensorWidth(
		float InSensorWidth // camera sensor width (in millimeters)
	);

	/** Get camera sensor width */
	float GetSensorWidth() const;
	
	// Set the aspect ratio of the Datasmith camera.
	void SetAspectRatio(
		float InAspectRatio // camera aspect ratio (width/height)
	);

	/** Get Datasmith camera aspect ratio (width/height) */
	float GetAspectRatio() const;

	// Set the Datasmith camera focus distance with the current camera world position and a target world position.
	void SetFocusDistance(
		float InTargetX, // target position on the X axis
		float InTargetY, // target position on the Y axis
		float InTargetZ  // target position on the Z axis
	);

	// Set the Datasmith camera focus distance.
	void SetFocusDistance(
		float InFocusDistance // camera focus distance (in world units)
	);

	/** Get camera focus distance */
	float GetFocusDistance() const;
	
	// Set the Datasmith camera focal length.
	void SetFocalLength(
		float InFocalLength // camera focal length (in millimeters)
	);

	// Set the Datasmith camera focal length.
	void SetFocalLength(
		float InFOV,         // camera field of view (in degrees)
		bool  bInVerticalFOV // whether or not the field of view value represents the camera view height
	);

	/** Get camera focal length in millimeters */
	float GetFocalLength() const;

	// Set the Datasmith camera look-at actor.
	void SetLookAtActor(
		const TCHAR* InActorName // look-at actor name
	);

	// Get the Datasmith camera look-at actor.
	const TCHAR* GetLookAtActor() const;

	// Set the Datasmith camera look-at roll allowed state.
	void SetLookAtAllowRoll(
		bool bInAllow // allow roll when look-at is active
	);

	// Get the Datasmith camera look-at actor.
	bool GetLookAtAllowRoll() const;

	/** The focus method of the camera, either None (no DoF) or Manual */
	bool GetEnableDepthOfField() const;

	/** The focus method of the camera, either None (no DoF) or Manual */
	void SetEnableDepthOfField(
		bool bInEnableDepthOfField
	);

	/** Get camera FStop also known as FNumber */
	float GetFStop() const;

	/** Set camera FStop also known as FNumber */
	void SetFStop(
		float InFStop
	);

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeActorCamera(
		const TSharedRef<IDatasmithCameraActorElement>& InInternalElement
	);

	TSharedRef<IDatasmithCameraActorElement> GetDatasmithActorCameraElement() const;
};
