// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeActor.h"
#include "DatasmithFacadeMaterialID.h"

class IDatasmithLightActorElement;
class IDatasmithPointLightElement;

class DATASMITHFACADE_API FDatasmithFacadeActorLight : public FDatasmithFacadeActor
{
public:
	virtual ~FDatasmithFacadeActorLight() {}

	/** Return true on light enabled, false otherwise */
	bool IsEnabled() const;

	/** Set enable property of the light */
	void SetEnabled(
		bool bIsEnabled
	);

	/** Get light intensity */
	double GetIntensity() const;

	/** Set light intensity */
	void SetIntensity(
		double Intensity
	);

	/** Get light color in sRGB mode */
	void GetColor(
		uint8& OutR,
		uint8& OutG,
		uint8& OutB,
		uint8& OutA
	) const;

	/** Get light color on linear mode */
	void GetColor(
		float& OutR,
		float& OutG,
		float& OutB,
		float& OutA
	) const;

	/** Set the Datasmith light sRGBA color. */
	void SetColor(
		uint8 InR,
		uint8 InG,
		uint8 InB,
		uint8 InA
	);

	/** Set the Datasmith light linear color. */
	void SetColor(
		float InR,
		float InG,
		float InB,
		float InA
	);

	/** Get the light temperature in Kelvin */
	double GetTemperature() const;

	/** Set the light temperature in Kelvin */
	void SetTemperature(
		double Temperature
	);

	/** Get if the light color is controlled by temperature */
	bool GetUseTemperature() const;

	/** Set if the light color is controlled by temperature */
	void SetUseTemperature(
		bool bUseTemperature
	);

	/** Get the path of the Ies definition file */
	const TCHAR* GetIesFile() const;

	void WriteIESFile(
		const TCHAR* InIESFileFolder,
		const TCHAR* InIESFileName,
		const TCHAR* InIESData
	);

	/** Set the path of the Ies definition file */
	void SetIesFile(
		const TCHAR* IesFile
	);

	/** Set if this light is controlled by Ies definition file */
	bool GetUseIes() const;

	/** Get if this light is controlled by Ies definition file */
	void SetUseIes(
		bool bUseIes
	);

	/** Get the Ies brightness multiplier */
	double GetIesBrightnessScale() const;

	/** Set the Ies brightness multiplier */
	void SetIesBrightnessScale(
		double IesBrightnessScale
	);

	/** Get if the emissive amount of the ies is controlled by the brightness scale */
	bool GetUseIesBrightness() const;

	/** Set if the emissive amount of the ies is controlled by the brightness scale */
	void SetUseIesBrightness(
		bool bUseIesBrightness
	);

	/** Get the rotation applied to the IES shape in the form of quaternion members */
	void GetIesRotation(
		float& OutX,
		float& OutY,
		float& OutZ,
		float& OutW
	) const;

	/** Get the rotation applied to the IES shape in the form of euler angles */
	void GetIesRotation(
		float& OutPitch,
		float& OutYaw,
		float& OutRoll
	) const;

	/** Set the rotation to apply to the IES shape in Quaternion format */
	void SetIesRotation(
		float X,
		float Y,
		float Z,
		float W
	);

	/** Set the rotation to apply to the IES shape from euler angles */
	void SetIesRotation(
		float Pitch,
		float Yaw,
		float Roll
	);

	/**
	 *	Returns a new FDatasmithFacadeMaterialID pointing to emissive material on this light
	 *	If there is no child at the given index, returned value is nullptr.
	 *	The caller is responsible of deleting the returned object pointer.
	 **/
	FDatasmithFacadeMaterialID* GetNewLightFunctionMaterial();

	/** Set emissive material on this light */
	void SetLightFunctionMaterial(
		FDatasmithFacadeMaterialID& InMaterial
	);

	/** Set emissive material on this light */
	void SetLightFunctionMaterial(
		const TCHAR* InMaterialName
	);

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeActorLight(
		const TSharedRef<IDatasmithLightActorElement>& InInternalActor
	);

	TSharedRef<IDatasmithLightActorElement> GetDatasmithLightActorElement() const;
};

class DATASMITHFACADE_API FDatasmithFacadePointLight : public FDatasmithFacadeActorLight
{
public:
	// Possible Datasmith point light intensity units.
	// Copy of EDatasmithLightUnits from DatasmithCore DatasmithDefinitions.h.
	enum class EPointLightIntensityUnit
	{
		Unitless,
		Candelas,
		Lumens
	};

	FDatasmithFacadePointLight(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadePointLight() {}

	void SetIntensityUnits(
		EPointLightIntensityUnit InUnits
	);
	
	EPointLightIntensityUnit GetIntensityUnits() const;

	/** Get light radius, width in case of 2D light sources */
	float GetSourceRadius() const;

	/** Set light radius, width in case of 2D light sources */
	void SetSourceRadius(
		float SourceRadius
	);

	/** Get light length only affects 2D shaped lights */
	float GetSourceLength() const;

	/** Set light length only affects 2D shaped lights */
	void SetSourceLength(
		float SourceLength
	);

	/** Get attenuation radius in centimeters */
	float GetAttenuationRadius() const;

	/** Set attenuation radius in centimeters */
	void SetAttenuationRadius(
		float AttenuationRadius
	);

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadePointLight(
		const TSharedRef<IDatasmithPointLightElement>& InInternalActor
	);

private:

	TSharedRef<IDatasmithPointLightElement> GetDatasmithPointLightElement() const;
};

class DATASMITHFACADE_API FDatasmithFacadeSpotLight: public FDatasmithFacadePointLight
{
public:
	FDatasmithFacadeSpotLight(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadeSpotLight() {}

	/** Get the inner cone angle for spot lights in degrees */
	float GetInnerConeAngle() const;

	/** Set the inner cone angle for spot lights in degrees */
	void SetInnerConeAngle(
		float InnerConeAngle
	);

	/** Get the outer cone angle for spot lights in degrees */
	float GetOuterConeAngle() const;

	/** Set the outer cone angle for spot lights in degrees */
	void SetOuterConeAngle(
		float OuterConeAngle
	);

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeSpotLight(
		const TSharedRef<IDatasmithSpotLightElement>& InInternalActor
	);

private:

	TSharedRef<IDatasmithSpotLightElement> GetDatasmithSpotLightElement() const;
};

class DATASMITHFACADE_API FDatasmithFacadeDirectionalLight : public FDatasmithFacadeActorLight
{
public:
	FDatasmithFacadeDirectionalLight(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadeDirectionalLight() {}

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeDirectionalLight(
		const TSharedRef<IDatasmithDirectionalLightElement>& InInternalActor
	);
};

/**
 * An area light is an emissive shape (light shape) with a light component (light type)
 */
class DATASMITHFACADE_API FDatasmithFacadeAreaLight : public FDatasmithFacadeSpotLight
{
public:
	FDatasmithFacadeAreaLight(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadeAreaLight() {}

	// Possible Datasmith area light shapes.
	// Copy of EDatasmithLightShape from DatasmithCore DatasmithDefinitions.h.
	enum class EAreaLightShape : uint8
	{
		Rectangle,
		Disc,
		Sphere,
		Cylinder,
		None
	};

	// Possible Datasmith area light types.
	// Copy of EDatasmithAreaLightType from DatasmithCore DatasmithDefinitions.h.
	enum class EAreaLightType
	{
		Point,
		Spot,
		IES_DEPRECATED,
		Rect
	};

	/** Get the light shape Rectangle/Sphere/Disc/Cylinder */
	EAreaLightShape GetLightShape() const;

	/** Set the light shape Rectangle/Sphere/Disc/Cylinder */
	void SetLightShape(
		EAreaLightShape Shape
	);

	/** Set the type of light for an area light: Point/Spot/Rect */
	void SetLightType(
		EAreaLightType LightType
	);
	
	EAreaLightType GetLightType() const;

	/** Set the area light shape size on the Y axis */
	void SetWidth(
		float InWidth
	);
	
	float GetWidth() const;

	/** Set the area light shape size on the X axis */
	void SetLength(
		float InLength
	);
	
	float GetLength() const;

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeAreaLight(
		const TSharedRef<IDatasmithAreaLightElement>& InInternalActor
	);

	TSharedRef<IDatasmithAreaLightElement> GetDatasmithAreaLightElement() const;
};

/**
 * Represents a ALightmassPortal
 *
 * Use the actor scale to drive the portal dimensions
 */
class DATASMITHFACADE_API FDatasmithFacadeLightmassPortal : public FDatasmithFacadePointLight
{
public:
	FDatasmithFacadeLightmassPortal(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadeLightmassPortal() {}

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeLightmassPortal(
		const TSharedRef<IDatasmithLightmassPortalElement>& InInternalActor
	);
};