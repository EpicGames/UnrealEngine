// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeActor.h"

class FDatasmithFacadeMaterialID;

class DATASMITHFACADE_API FDatasmithFacadeActorMesh :
	public FDatasmithFacadeActor
{
public:

	FDatasmithFacadeActorMesh(
		const TCHAR* InElementName // Datasmith element name
	);

	virtual ~FDatasmithFacadeActorMesh() {}

	// Set the static mesh of the Datasmith mesh actor.
	void SetMesh(
		const TCHAR* InMeshName // Datasmith static mesh name
	);

	// Get the name of the StaticMesh associated with the actor
	const TCHAR* GetMeshName() const;

	// Adds a new material override to the Actor Element
	void AddMaterialOverride(
		const TCHAR* MaterialName //name of the material, it should be unique
		, int32 Id //material identifier to be used with mesh sub-material indices
	);

	// Adds a new material override to the Actor Element
	void AddMaterialOverride(
		FDatasmithFacadeMaterialID& Material
	);

	// Get the amount of material overrides on this mesh
	int32 GetMaterialOverridesCount() const;

	/**
	 *	Returns a new FDatasmithFacadeMaterialID pointing to the i-th material override of this actor.
	 *	If there is no child at the given index, returned value is nullptr.
	 *	The caller is responsible of deleting the returned object pointer.
	 */
	FDatasmithFacadeMaterialID* GetNewMaterialOverride(
		int32 MaterialOverrideIndex
	);

	// Remove material from the Actor Element
	void RemoveMaterialOverride(
		FDatasmithFacadeMaterialID& Material
	);

	/**
	 *  Remove all material overrides from the Actor Element.
	 */
	void ResetMaterialOverrides();

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeActorMesh(
		const TSharedRef<IDatasmithMeshActorElement>& InInternalElement
	);

	TSharedRef<IDatasmithMeshActorElement> GetDatasmithMeshActorElement() const;
};
