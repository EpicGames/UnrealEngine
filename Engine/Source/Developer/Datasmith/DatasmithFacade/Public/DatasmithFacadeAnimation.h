// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithFacadeElement.h"

enum class EDatasmithFacadeCurveInterpMode
{
	Linear,
	Constant,
	Cubic
};

enum class EDatasmithFacadeAnimationTransformType
{
	Translation,
	Rotation,
	Scale
};

enum class EDatasmithFacadeElementAnimationSubType : uint64
{
	BaseAnimation = 0,
	TransformAnimation = 1 << 0,
	VisibilityAnimation = 1 << 1,
	SubsequenceAnimation = 1 << 2,
};

enum class EDatasmithFacadeCompletionMode : uint8
{
	KeepState,
	RestoreState,
	ProjectDefault,
};

enum class EDatasmithFacadeTransformChannels : uint16
{
	None			= 0x000,

	TranslationX 	= 0x001,
	TranslationY 	= 0x002,
	TranslationZ 	= 0x004,
	Translation 	= TranslationX | TranslationY | TranslationZ,

	RotationX 		= 0x008,
	RotationY 		= 0x010,
	RotationZ 		= 0x020,
	Rotation 		= RotationX | RotationY | RotationZ,

	ScaleX 			= 0x040,
	ScaleY 			= 0x080,
	ScaleZ 			= 0x100,
	Scale 			= ScaleX | ScaleY | ScaleZ,

	All				= Translation | Rotation | Scale,
};

class DATASMITHFACADE_API FDatasmithFacadeBaseAnimation : public FDatasmithFacadeElement
{
public:
	bool IsSubType( const EDatasmithFacadeElementAnimationSubType AnimSubType ) const;

	/** Set how the actor should behave once its animation completes */
	void SetCompletionMode(EDatasmithFacadeCompletionMode CompletionMode);

	/** Get how the actor behaves once this animation is complete */
	EDatasmithFacadeCompletionMode GetCompletionMode() const;

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadeBaseAnimation(
		const TSharedRef<IDatasmithBaseAnimationElement>& InInternalAnimation
	);

	TSharedRef<IDatasmithBaseAnimationElement> GetDatasmithBaseAnimation() const;
};

class DATASMITHFACADE_API FDatasmithFacadeTransformAnimation : public FDatasmithFacadeBaseAnimation
{
public:
	FDatasmithFacadeTransformAnimation(const TCHAR* InName);

	/** Add a frame of the given transform type to the animation */
	void AddFrame(EDatasmithFacadeAnimationTransformType TransformType, int32 FrameNumber, float X, float Y, float Z);

	/** Return the number of frames of the given transform type in the animation */
	int32 GetFramesCount(EDatasmithFacadeAnimationTransformType TransformType) const;

	/** Set the interpolation mode of the given transform type in the animation */
	void SetCurveInterpMode(EDatasmithFacadeAnimationTransformType TransformType, EDatasmithFacadeCurveInterpMode CurveInterpMode);

	/** Get the interpolation mode of the given transform type in the animation */
	EDatasmithFacadeCurveInterpMode GetCurveInterpMode(EDatasmithFacadeAnimationTransformType TransformType) const;

	/** Return the frame of the given transform type at the given index or an invalid frame if the index was out of bounds */
	void GetFrame(EDatasmithFacadeAnimationTransformType TransformType, int32 Index, int32& OutFrameNumber, double& OutX, double& OutY, double& OutZ) const;

	/** Remove the frame of the given transform type at the given index from the animation */
	void RemoveFrame(EDatasmithFacadeAnimationTransformType TransformType, int32 Index);

	/** Gets which channels of this animation will be moved to the animation assets or serialized. All channels are enabled by default */
	EDatasmithFacadeTransformChannels GetEnabledTransformChannels() const;

	/** Sets which channels of this animation will be moved to the animation assets or serialized. All channels are enabled by default */
	void SetEnabledTransformChannels(EDatasmithFacadeTransformChannels Channels);

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadeTransformAnimation(
		const TSharedRef<IDatasmithTransformAnimationElement>& InInternalAnimation
	);

	TSharedRef<IDatasmithTransformAnimationElement> GetDatasmithTransformAnimation() const;
};

class DATASMITHFACADE_API FDatasmithFacadeVisibilityAnimation : public FDatasmithFacadeBaseAnimation
{
public:
	FDatasmithFacadeVisibilityAnimation(const TCHAR* InName);

	/** Add a frame to the animation */
	void AddFrame(int32 FrameNumber, bool bVisible);

	/** Return the number of frames of the animation */
	int32 GetFramesCount() const;

	/** Set the interpolation mode of the animation */
	void SetCurveInterpMode(EDatasmithFacadeCurveInterpMode CurveInterpMode);

	/** Get the interpolation mode of the animation */
	EDatasmithFacadeCurveInterpMode GetCurveInterpMode() const;

	/** Return the frame of the animation at the given index or an invalid frame if the index was out of bounds */
	void GetFrame(int32 Index, int32& OutFrameNumber, bool& bOutVisible) const;

	/** Remove the frame at the given index from the animation */
	void RemoveFrame(int32 Index);

	/** Sets whether this animation will be duplicated to all children (recursively) when imported */
	void SetPropagateToChildren(bool bPropagate);

	/** Gets whether this animation will be duplicated to all children (recursively) when imported */
	bool GetPropagateToChildren() const;

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadeVisibilityAnimation(
		const TSharedRef<IDatasmithVisibilityAnimationElement>& InInternalAnimation
	);

	TSharedRef<IDatasmithVisibilityAnimationElement> GetDatasmithVisibilityAnimation() const;
};

class FDatasmithFacadeLevelSequence;

class DATASMITHFACADE_API FDatasmithFacadeSubsequenceAnimation : public FDatasmithFacadeBaseAnimation
{
public:
	FDatasmithFacadeSubsequenceAnimation(const TCHAR* InName);

	/** Get the frame where the subsequence starts */
	int32 GetStartTime() const;

	/** Set the frame where the subsequence starts */
	void SetStartTime(int32 InFrameNumber);

	/** Get the subsequence duration in number of frames */
	int32 GetDuration() const;

	/** Set the subsequence duration in number of frames */
	void SetDuration(int32 InDuration);

	/** Get the time scale used for the subsequence */
	float GetTimeScale() const;

	/** Set the time scale used for the subsequence */
	void SetTimeScale(float InTimeScale);

	/** Get a pointer to the subsequence that this element references */
	FDatasmithFacadeLevelSequence* GetNewSubsequence() const;

	/** Set the subsequence that this element references */
	void SetSubsequence(FDatasmithFacadeLevelSequence* InSubsequence);

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadeSubsequenceAnimation(
		const TSharedRef<IDatasmithSubsequenceAnimationElement>& InInternalAnimation
	);

	TSharedRef<IDatasmithSubsequenceAnimationElement> GetDatasmithSubsequenceAnimation() const;
};

class DATASMITHFACADE_API FDatasmithFacadeLevelSequence : public FDatasmithFacadeElement
{
public:
	FDatasmithFacadeLevelSequence(const TCHAR* InName);

	/** Get the output filename, it can be absolute or relative to the scene file */
	const TCHAR* GetFile() const;

	/** Set the output filename, it can be absolute or relative to the scene file */
	void SetFile(const TCHAR* InFile);

	/** Return a MD5 hash of the content of the Level Sequence Element. Used in CalculateElementHash to quickly identify Element with identical content */
	void GetFileHash(TCHAR OutBuffer[33], size_t BufferSize) const;

	/** Set the MD5 hash of the Level Sequence file. This should be a hash of its content. */
	void SetFileHash(const TCHAR* Hash);

	/* Gets the frame rate for the animations in the level sequence */
	float GetFrameRate() const;

	/* Sets the frame rate for the animations in the level sequence */
	void SetFrameRate(float FramePerSecs);

	/**
	 * Adds an animation to the level sequence.
	 *
	 * @param InAnimation the animation to add
	 */
	void AddAnimation(FDatasmithFacadeTransformAnimation* InAnimation);

	/** Returns the number of animations in the level sequence */
	int32 GetAnimationsCount() const;

	/** Returns the animation using this index */
	FDatasmithFacadeTransformAnimation* GetNewTransformAnimation(int32 InIndex);
	FDatasmithFacadeVisibilityAnimation* GetNewVisibilityAnimation(int32 InIndex);
	FDatasmithFacadeSubsequenceAnimation* GetNewSubsequenceAnimation(int32 InIndex);

	/**
	 * Removes an animation from the level sequence.
	 *
	 * @param InAnimation the animation to remove
	 */
	void RemoveAnimation(FDatasmithFacadeBaseAnimation* InAnimation);

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadeLevelSequence(
		const TSharedRef<IDatasmithLevelSequenceElement>& InInternalLevelSequence
	);

	TSharedRef<IDatasmithLevelSequenceElement> GetDatasmithLevelSequence() const;
};

