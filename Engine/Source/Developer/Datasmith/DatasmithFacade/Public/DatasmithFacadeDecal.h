// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeActor.h"
#include "DatasmithFacadeMaterial.h"

class DATASMITHFACADE_API FDatasmithFacadeActorDecal : public FDatasmithFacadeActor
{
	friend class FDatasmithFacadeScene;

public:
	FDatasmithFacadeActorDecal(
		const TCHAR* InElementName // Datasmith element name
	);

	virtual ~FDatasmithFacadeActorDecal() {}

	/** Get the Decal element size */
	void GetDimensions(double& OutX, double& OutY, double& OutZ) const;

	/** Set the Decal element size */
	void SetDimensions(double InX, double InY, double InZ);

	/** Get the path name of the Material associated with the actor */
	const TCHAR* GetDecalMaterialPathName() const;

	/** Set the path name of the Material that the Decal actor uses */
	void SetDecalMaterialPathName(const TCHAR*);

	/** Get the order in which Decal element is rendered */
	int32 GetSortOrder() const;

	/** Set the order in which decal elements are rendered.  Higher values draw later (on top) */
	void SetSortOrder(int32);

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeActorDecal(
		const TSharedRef<IDatasmithDecalActorElement>& InInternalActor
	);

	TSharedRef<IDatasmithDecalActorElement> GetDatasmithDecalActorElement() const;
};

class DATASMITHFACADE_API FDatasmithFacadeDecalMaterial : public FDatasmithFacadeBaseMaterial
{
	friend class FDatasmithFacadeScene;

public:

	FDatasmithFacadeDecalMaterial(
		const TCHAR* InElementName // Datasmith element name
	);

	virtual ~FDatasmithFacadeDecalMaterial() {}

	/** Get path name of the diffuse texture associated with the material */
	const TCHAR* GetDiffuseTexturePathName() const;

	/** Set path name of the diffuse texture associated with the material */
	void SetDiffuseTexturePathName(const TCHAR* DiffuseTexturePathName);

	/** Get path name of the normal texture associated with the material */
	const TCHAR* GetNormalTexturePathName() const;

	/** Set path name of the normal texture associated with the material */
	void SetNormalTexturePathName(const TCHAR* NormalTexturePathName);

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeDecalMaterial(
		const TSharedRef<IDatasmithDecalMaterialElement>& InMaterialRef
	);

	TSharedRef<IDatasmithDecalMaterialElement> GetDatasmithDecalMaterial() const;
};
