// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith SDK.
#include "DatasmithSceneFactory.h"

class FDatasmithFacadeScene;

class DATASMITHFACADE_API FDatasmithFacadeElement
{
	friend class FDatasmithFacadeScene;

public:

	// Possible coordinate system types.
	enum class ECoordinateSystemType
	{
		LeftHandedYup, // left-handed Y-up coordinate system: Cinema 4D
		LeftHandedZup, // left-handed Z-up coordinate system: Unreal native
		RightHandedZup // right-handed Z-up coordinate system: Revit
	};

public:

	// Set the coordinate system type of the world geometries and transforms.
	static void SetCoordinateSystemType(
		ECoordinateSystemType InWorldCoordinateSystemType // world coordinate system type
	);

	/**
	 * Set the scale factor from world units to Datasmith centimeters.
	 * If the given value is too close to 0, it is clamped to SMALL_NUMBER value.
	 */
	static void SetWorldUnitScale(
		float InWorldUnitScale // scale factor from world units to Datasmith centimeters
	);

	virtual ~FDatasmithFacadeElement() {}

	// Hash the given InString and fills the OutBuffer up to the BufferSize. A string hash has 32 character (+ null character).
	static void GetStringHash( const TCHAR* InString, TCHAR OutBuffer[33], size_t BufferSize );

	// Set the Datasmith element name.
	void SetName(
		const TCHAR* InElementName // Datasmith element name
	);

	// Return the Datasmith element name.
	const TCHAR* GetName() const;

	// Set the Datasmith element label.
	void SetLabel(
		const TCHAR* InElementLabel // Datasmith element label
	);

	// Return the Datasmith element label.
	const TCHAR* GetLabel() const;

#ifdef SWIG_FACADE
protected:
#endif

	template<bool kIsForward, bool kIsDirection, typename Vec_t> static Vec_t Convert(const Vec_t& V);

	static FVector   ConvertDirection    (const FVector& In);
	static FVector   ConvertBackDirection(const FVector& In);
	static FVector   ConvertPosition     (const FVector& In);
	static FVector   ConvertBackPosition (const FVector& In);

	static FVector   ConvertDirection    (double InX, double InY, double InZ);
	static FVector   ConvertBackDirection(double InX, double InY, double InZ);
	static FVector   ConvertPosition     (double InX, double InY, double InZ);
	static FVector   ConvertBackPosition (double InX, double InY, double InZ);

	static FVector3f ConvertDirection    (const FVector3f& In);
	static FVector3f ConvertBackDirection(const FVector3f& In);
	static FVector3f ConvertPosition     (const FVector3f& In);
	static FVector3f ConvertBackPosition (const FVector3f& In);

	static FVector3f ConvertDirection    (float InX, float InY, float InZ);
	static FVector3f ConvertBackDirection(float InX, float InY, float InZ);
	static FVector3f ConvertPosition     (float InX, float InY, float InZ);
	static FVector3f ConvertBackPosition (float InX, float InY, float InZ);

	static FVector   ConvertTranslation(FVector const& InVertex) { return ConvertPosition(InVertex); }
	static FVector3f ConvertTranslation(FVector3f const& InVertex) { return ConvertPosition(InVertex); }

	// Build and export the Datasmith scene element asset when required.
	// This must be done before building a Datasmith scene element.
	virtual void ExportAsset(
		FString const& InAssetFolder // Datasmith asset folder path
	);

	TSharedRef<IDatasmithElement>& GetDatasmithElement() { return InternalDatasmithElement;	}

	const TSharedRef<IDatasmithElement>& GetDatasmithElement() const { return InternalDatasmithElement;	}

protected:

	FDatasmithFacadeElement(
		const TSharedRef<IDatasmithElement>& InElement
	);

	// Coordinate system type of the world geometries and transforms.
	static ECoordinateSystemType WorldCoordinateSystemType;

	// Scale factor from world units to Datasmith centimeters.
	static float WorldUnitScale;

	TSharedRef<IDatasmithElement> InternalDatasmithElement;
};
