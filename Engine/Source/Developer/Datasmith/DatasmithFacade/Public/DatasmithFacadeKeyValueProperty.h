// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeElement.h"

class IDatasmithKeyValueProperty;

class DATASMITHFACADE_API FDatasmithFacadeKeyValueProperty : public FDatasmithFacadeElement
{
public:
	enum class EKeyValuePropertyType
	{
		String,
		Color,
		Float,
		Bool,
		Texture,
		Vector
	};

	FDatasmithFacadeKeyValueProperty( const TCHAR* InName );

	virtual ~FDatasmithFacadeKeyValueProperty() {}

	/** Get the type of this property */
	EKeyValuePropertyType GetPropertyType() const
	{
		return static_cast<EKeyValuePropertyType>(GetDatasmithKeyValueProperty()->GetPropertyType());
	}

	/** Set the type of this property */
	void SetPropertyType( EKeyValuePropertyType InType )
	{
		GetDatasmithKeyValueProperty()->SetPropertyType( static_cast<EDatasmithKeyValuePropertyType>(InType) );
	}

	/** Get the value of this property */
	const TCHAR* GetValue() const
	{
		return GetDatasmithKeyValueProperty()->GetValue();
	}

	/** Sets the value of this property */
	void SetValue( const TCHAR* Value )
	{
		GetDatasmithKeyValueProperty()->SetValue( Value );
	}

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeKeyValueProperty( const TSharedRef<IDatasmithKeyValueProperty>& InKeyValueProperty );

	TSharedRef<IDatasmithKeyValueProperty> GetDatasmithKeyValueProperty() const
	{
		return StaticCastSharedRef<IDatasmithKeyValueProperty>( GetDatasmithElement() );
	}
};