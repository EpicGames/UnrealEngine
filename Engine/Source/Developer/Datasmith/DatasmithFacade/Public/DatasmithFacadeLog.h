// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith SDK.
#include "Logging/LogMacros.h"
#include "Containers/UnrealString.h"
DECLARE_LOG_CATEGORY_EXTERN(LogDatasmithFacade, Log, All);


class DATASMITHFACADE_API FDatasmithFacadeLog
{
public:

	FDatasmithFacadeLog();

	// Append a text line to the log string.
	void AddLine(
		const TCHAR* InLine // text line to append (without a newline)
	);

	// Add one level of indentation for further logged text lines.
	void MoreIndentation();

	// Remove one level of indentation for further logged text lines.
	void LessIndentation();

	// Write the multi-line log string into a text file.
	void WriteFile(
		const TCHAR* InFilePath // text file path
	) const;

private:

	// Multi-line log string.
	FString Log;

	// Number of levels of indentation for logged text lines.
	int LineIndentation;
};
