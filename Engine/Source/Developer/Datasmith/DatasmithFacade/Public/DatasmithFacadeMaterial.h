// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeElement.h"
#include "DatasmithFacadeTexture.h"

class FDatasmithFacadeKeyValueProperty;

class DATASMITHFACADE_API FDatasmithFacadeBaseMaterial :
	public FDatasmithFacadeElement
{
public:
	enum class EDatasmithMaterialType
	{
		MaterialInstance,
		UEPbrMaterial,
		DecalMaterial,
		Unsupported,
	};

	EDatasmithMaterialType GetDatasmithMaterialType() const;

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeBaseMaterial(
		const TSharedRef<IDatasmithBaseMaterialElement>& BaseMaterialElement
	);

	static EDatasmithMaterialType GetDatasmithMaterialType(
		const TSharedRef<IDatasmithBaseMaterialElement>& InMaterial
	);

	TSharedRef<IDatasmithBaseMaterialElement> GetDatasmithBaseMaterial() const;

	static FDatasmithFacadeBaseMaterial* GetNewFacadeBaseMaterialFromSharedPtr(
		const TSharedPtr<IDatasmithBaseMaterialElement>& InMaterial
	);
};

class DATASMITHFACADE_API FDatasmithFacadeMaterialInstance :
	public FDatasmithFacadeBaseMaterial
{
	friend class FDatasmithFacadeScene;

public:

	// Possible Datasmith material types, from EDatasmithMaterialInstanceType in DatasmithDefinitions.h
	enum class EMaterialInstanceType
	{
		Auto,
		Opaque,
		Transparent,
		ClearCoat,
		Custom,
		/** Material has a transparent cutout map */
		CutOut,
		Emissive,
		Decal,
		/** Dummy element to count the number of types */
		Count
	};

	enum class EMaterialInstanceQuality : uint8
	{
		High,
		Low,
		/** Dummy element to count the number of qualities */
		Count
	};

public:

	FDatasmithFacadeMaterialInstance(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadeMaterialInstance() {}

	EMaterialInstanceType GetMaterialType() const;

	void SetMaterialType(
		EMaterialInstanceType InMaterialInstanceType
	);

	EMaterialInstanceQuality GetQuality() const;

	void SetQuality(
		EMaterialInstanceQuality InQuality
	);

	/** Get the material path name used when material type is set to Custom */
	const TCHAR* GetCustomMaterialPathName() const;

	/** Set the material path name used when material type is set to Custom */
	void SetCustomMaterialPathName(
		const TCHAR* InPathName
	);

	// Add a Datasmith material sRGBA color property.
	void AddColor(
		const TCHAR*  InPropertyName, // color property name
		unsigned char InR,            // red
		unsigned char InG,            // green
		unsigned char InB,            // blue
		unsigned char InA             // alpha
	);

	// Add a Datasmith material linear color property.
	void AddColor(
		const TCHAR* InPropertyName, // color property name
		float        InR,            // red
		float        InG,            // green
		float        InB,            // blue
		float        InA             // alpha
	);

	// Add a Datasmith material texture property.
	void AddTexture(
		const TCHAR* InPropertyName,             // texture property name
		const FDatasmithFacadeTexture* InTexture // texture file path
	);

	// Add a Datasmith material string property.
	void AddString(
		const TCHAR* InPropertyName, // property name
		const TCHAR* InPropertyValue // property value
	);

	// Add a Datasmith material float property.
	void AddFloat(
		const TCHAR* InPropertyName, // property name
		float        InPropertyValue // property value
	);

	// Add a Datasmith material boolean property.
	void AddBoolean(
		const TCHAR* InPropertyName,  // property name
		bool         bInPropertyValue // property value
	);

	int32 GetPropertiesCount() const;

	/** Returns a new FDatasmithFacadeKeyValueProperty pointing to the KeyValueProperty at the given index, the returned value must be deleted after used, can be nullptr. */
	FDatasmithFacadeKeyValueProperty* GetNewProperty(
		int32 PropertyIndex
	) const;

	/** Returns a new FDatasmithFacadeKeyValueProperty pointing to the KeyValueProperty with the given name, the returned value must be deleted after used, can be nullptr. */
	FDatasmithFacadeKeyValueProperty* GetNewPropertyByName(
		const TCHAR* PropertyName
	) const;

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeMaterialInstance(
		const TSharedRef<IDatasmithMaterialInstanceElement>& InMaterialRef
	);

	TSharedRef<IDatasmithMaterialInstanceElement> GetDatasmithMaterialInstance() const;
};
