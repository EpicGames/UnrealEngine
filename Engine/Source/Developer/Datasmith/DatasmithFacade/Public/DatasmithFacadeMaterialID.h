// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeElement.h"


class DATASMITHFACADE_API FDatasmithFacadeMaterialID : 
	public FDatasmithFacadeElement
{
public:
	FDatasmithFacadeMaterialID(
		const TCHAR* MaterialIDName
	);

	virtual ~FDatasmithFacadeMaterialID() {}

	void SetId(
		int32 Id
	);
	
	int32 GetId() const;

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeMaterialID(
		const TSharedRef<IDatasmithMaterialIDElement>& InMaterialElement
	);

	TSharedRef<IDatasmithMaterialIDElement> GetMaterialIDElement() const;
};