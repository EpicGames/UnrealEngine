// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeTexture.h"
#include "DatasmithFacadeUEPbrMaterial.h"

#include "CoreMinimal.h"
#include "DatasmithMaterialsUtils.h"

class IDatasmithExpressionInput;
class IDatasmithUEPbrMaterialElement;
class IDatasmithMaterialExpression;
class IDatasmithExpressionParameter;

class FDatasmithFacadeMaterialExpression;

class DATASMITHFACADE_API FDatasmithFacadeMaterialsUtils
{
public:
	//This is a static only class.
	FDatasmithFacadeMaterialsUtils() = delete;
	
	struct FUVEditParameters
	{
		// UV space has its origin at top-left, with U going left-to-right, V top-to-bottom
		void SetUVOffset( float X, float Y );
		void GetUVOffset( float& OutX, float& OutY ) const;

		//FVector2D UVTiling;
		void SetUVTiling( float X, float Y );
		void GetUVTiling( float& OutX, float& OutY ) const;

		// W rotation center
		void SetRotationPivot( float X, float Y, float Z );
		void GetRotationPivot( float& OutX, float& OutY, float& OutZ ) const;

		// W rotation in degrees (rotation is counterclockwise)
		void SetRotationAngle( float Angle );
		float GetRotationAngle() const;

		// UV channel to use
		void SetChannelIndex( uint8 ChannelIndex );
		uint8 GetChannelIndex() const;

		// Enable "Real-World Scale" behavior as in 3ds max
		void SetIsUsingRealWorldScale( bool bIsUsingRealWorldScale );
		bool GetIsUsingRealWorldScale() const;

		// Enable mirroring of texture in U and V
		void SetMirrorU( bool bMirrorU );
		bool GetMirrorU() const;
		void SetMirrorV( bool bMirrorV );
		bool GetMirrorV() const;

#ifdef SWIG_FACADE
	protected:
#endif
		DatasmithMaterialsUtils::FUVEditParameters UVEditParameters;
	};

	/**
	 * Generate material expressions on a given DatasmithUEPbrMaterialElement to output a texture with UV settings applied to it
	 * The returned pointer must be deleted after use.
	 * @param MaterialElement		The UEPbrMaterialElement on which to create the material expressions
	 * @param ParameterName			The display name for the Texture expression
	 * @param TextureMapPath		The texture to use for the Texture expression
	 * @param UVParameters			The UVEditParameters to apply
	 * @return						Texture expression with UVEdit applied to it; nullptr if no TextureMapPath is specified
	 */
	static FDatasmithFacadeMaterialExpressionTexture* CreateNewFacadeTextureExpression( FDatasmithFacadeUEPbrMaterial& MaterialElement, const TCHAR* ParameterName, const TCHAR* TextureMapPath, const FUVEditParameters& UVParameters );

	// Helper struct used for reproducing the behavior of the optional value-type in interfaced language.
	struct FWeightedMaterialExpressionParameters
	{
	public:
		FWeightedMaterialExpressionParameters( float InWeight )
			: Weight( InWeight )
			, TextureMode( FDatasmithFacadeTexture::ETextureMode::Diffuse )
			, Expression( nullptr, TSharedPtr<IDatasmithUEPbrMaterialElement>() )
		{}

		// Set the color parameter taking linear channel input.
		void SetColor( float R, float G, float B, float A );
		// Set the color parameter taking non-linear sRGB channel input.
		void SetColorsRGB( uint8 R, uint8 G, uint8 B, uint8 A );

		void SetScalar( float Value );

		void SetTextureMode( FDatasmithFacadeTexture::ETextureMode InTextureMode );

		void SetExpression( const FDatasmithFacadeMaterialExpression& InExpression );

#ifdef SWIG_FACADE
	protected:
#endif
		// bHasColor used to emulate TOptional<> behavior, if bHasColor is false, the Color parameter will be ignored.
		TOptional<FLinearColor> Color;
		
		// bHasScalar used to emulate TOptional<> behavior, if bHasColor is false, the Scalar parameter will be ignored.
		TOptional<float> Scalar;
			
		float Weight;

		FDatasmithFacadeTexture::ETextureMode TextureMode;

		// Local copy of the optional FDatasmithFacadeMaterialExpression.
		// We shall not create a getter for this member as Expression may hold invalid pointer to the actual IDatasmithMaterialExpression.
		FDatasmithFacadeMaterialExpression Expression;
	};

	/**
	 * Generate material expressions on a given FDatasmithFacadeUEPbrMaterial that output a color or scalar interpolated with a MaterialExpression.
	 * The returned pointer must be deleted after use.
	 * @param MaterialElement				The UEPbrMaterialElement on which to create the material expressions
	 * @param ParameterName					The display name for the Color or Scalar input
	 * @param WeightedExpressionParamater	The structure defining the parameters of this weighted expression.				
	 */
	static FDatasmithFacadeMaterialExpression* CreateNewFacadeWeightedMaterialExpression( FDatasmithFacadeUEPbrMaterial& MaterialElement, const TCHAR* ParameterName, FWeightedMaterialExpressionParameters& WeightedExpressionParameter );

	/**
	 * Create a new FDatasmithFacadeTexture and automatically fill its properties from the given arguments.
	 * The returned pointer can be null if the path in empty and must be deleted after use.
	 *
	 * @param InTextureFilePath	The file path of the texture, used to determine the texture name and label as well.
	 * @param InTextureMode		The mode of the texture, used to determine the texture name and label as well.
	 */
	static FDatasmithFacadeTexture* CreateSimpleTextureElement( const TCHAR* InTextureFilePath, FDatasmithFacadeTexture::ETextureMode InTextureMode = FDatasmithFacadeTexture::ETextureMode::Diffuse );
};