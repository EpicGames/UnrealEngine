// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeElement.h"
#include "DatasmithMesh.h"

// Datasmith SDK classes.
class FDatasmithFacadeMaterialID;
class IDatasmithMeshElement;

class DATASMITHFACADE_API FDatasmithFacadeMesh
{
public:

	FDatasmithFacadeMesh()
		: RootOwnerMesh(MakeShared<FDatasmithMesh>())
		, InternalMesh(RootOwnerMesh.Get())
	{}

	void SetName(
		const TCHAR* InName
	)
	{
		GetDatasmithMesh().SetName(InName);
	}

	const TCHAR* GetName() const
	{
		return GetDatasmithMesh().GetName();
	}

	/** Return a MD5 hash of the content of the Datasmith Mesh */
	void CalculateHash(TCHAR OutBuffer[33], size_t BufferSize) const;

	//--------------------------
	// Faces
	//--------------------------
	/** Setting the amount of faces is mandatory before filling the array */
	void SetFacesCount(
		int32 NumFaces
	)
	{
		GetDatasmithMesh().SetFacesCount(NumFaces);
	}

	/** Retrieves the amount of faces */
	int32 GetFacesCount() const
	{
		return GetDatasmithMesh().GetFacesCount();
	}

	/**
	 * Sets the geometry of the face
	 *
	 * @param Index		value to choose the face that will be affected
	 * @param Vertex1	Index of the first geometric vertex that defines this face
	 * @param Vertex2	Index of the second geometric vertex that defines this face
	 * @param Vertex3	Index of the third geometric vertex that defines this face
	 */
	void SetFace(
		int32 Index,
		int32 Vertex1,
		int32 Vertex2,
		int32 Vertex3,
		int32 MaterialId = 0
	)
	{
		GetDatasmithMesh().SetFace(Index, Vertex1, Vertex2, Vertex3, MaterialId);
	}

	void GetFace(
		int32 Index,
		int32& OutVertex1,
		int32& OutVertex2,
		int32& OutVertex3,
		int32& OutMaterialId
	) const
	{
		GetDatasmithMesh().GetFace(Index, OutVertex1, OutVertex2, OutVertex3, OutMaterialId);
	}

	/**
	 * Sets the smoothing mask of the face
	 *
	 * @param Index			Index of the face that will be affected
	 * @param SmoothingMask	32 bits mask, 0 means no smoothing
	 */
	void SetFaceSmoothingMask(
		int32 Index,
		uint32 SmoothingMask
	)
	{
		GetDatasmithMesh().SetFaceSmoothingMask(Index, SmoothingMask);
	}

	/**
	 * Gets the smoothing mask of a face
	 *
	 * @param Index		Index of the face to retrieve the smoothing mask from
	 */
	uint32 GetFaceSmoothingMask(
		int32 Index
	) const
	{
		return GetDatasmithMesh().GetFaceSmoothingMask(Index);
	}

	int32 GetMaterialsCount() const
	{
		return GetDatasmithMesh().GetMaterialsCount();
	}

	bool IsMaterialIdUsed(
		int32 MaterialId
	) const
	{
		return GetDatasmithMesh().IsMaterialIdUsed(MaterialId);
	}

	//--------------------------
	// Vertices
	//--------------------------
	/** Setting the amount of geometric vertices is mandatory before filling the array */
	void SetVerticesCount(
		int32 NumVerts
	)
	{
		GetDatasmithMesh().SetVerticesCount(NumVerts);
	}

	/** Retrieves the amount of geometric vertices. The validity of the vertex data is not guaranteed */
	int32 GetVerticesCount() const
	{
		return GetDatasmithMesh().GetVerticesCount();
	}

	/**
	 * Sets the 3d position of the vertex
	 *
	 * @param Index value to choose the vertex that will be affected
	 * @x position on the x axis
	 * @y position on the y axis
	 * @z position on the z axis
	 */
	void SetVertex(
		int32 Index,
		float X,
		float Y,
		float Z
	);

	void GetVertex(
		int32 Index,
		float& OutX,
		float& OutY,
		float& OutZ
	) const;

	//--------------------------
	// Normals
	//--------------------------
	/**
	 * Sets the 3d normal
	 *
	 * @param Index value to choose the normal that will be affected
	 * @x direction on the x axis
	 * @y direction on the y axis
	 * @z direction on the z axis
	 */
	void SetNormal(
		int32 Index,
		float X,
		float Y,
		float Z
	);

	void GetNormal(
		int32 Index,
		float& OutX,
		float& OutY,
		float& OutZ
	) const;

	//--------------------------
	// UVs
	//--------------------------
	/**
	 * Sets the amount of UV channels on this mesh
	 *
	 * @param ChannelCount	The number of UV channels
	 */
	void SetUVChannelsCount(
		int32 ChannelCount
	)
	{
		GetDatasmithMesh().SetUVChannelsCount(ChannelCount);
	}

	/**
	 * Add a UV channel at the end
	 */
	void AddUVChannel()
	{
		GetDatasmithMesh().AddUVChannel();
	}

	/**
	 * Remove the last UV channel
	 */
	void RemoveUVChannel()
	{
		GetDatasmithMesh().RemoveUVChannel();
	}

	/** Gets the amount of UV channels on this mesh */
	int32 GetUVChannelsCount() const
	{
		return GetDatasmithMesh().GetUVChannelsCount();
	}

	/**
	 * Setting the amount of UV coordinates on the channel is mandatory before filling the array
	 *
	 * @param Channel	The channel to set the numbers of UVs
	 * @param NumVerts	The number of UVs in the channel
	 */
	void SetUVCount(
		int32 Channel,
		int32 NumVerts
	)
	{
		return GetDatasmithMesh().SetUVCount(Channel, NumVerts);
	}

	/** Retrieves the amount of UV coordinates on the channel. The validity of the vertex data is not guaranteed */
	int32 GetUVCount(
		int32 Channel
	) const
	{
		return GetDatasmithMesh().GetUVCount(Channel);
	}

	/**
	 * Sets the 2d position of the UV vertex for the first uv mapping
	 *
	 * @param Index	Value to choose the vertex that will be affected
	 * @param U		Horizontal coordinate
	 * @param V		Vertical coordinate
	 */
	void SetUV(
		int32 Channel,
		int32 Index,
		double U,
		double V
	)
	{
		return GetDatasmithMesh().SetUV(Channel, Index, U, V);
	}

	/**
	 * Get the hash for a uv channel
	 */
	uint32 GetHashForUVChannel(
		int32 Channel
	) const
	{
		return GetDatasmithMesh().GetHashForUVChannel(Channel);
	}

	/**
	 * Gets the UV coordinates for a channel
	 *
	 * @param Channel	The channel we want to retrieve the UVs from
	 * @param Index		The UV coordinates to retrieve
	 */
	void GetUV(
		int32 Channel,
		int32 Index,
		double& OutU,
		double& OutV
	) const
	{
		FVector2D UV(GetDatasmithMesh().GetUV(Channel, Index));
		OutU = UV.X;
		OutV = UV.Y;
	}

	/**
	 * Sets the channel UV coordinates of the face
	 *
	 * @param Index		Index of the face that will be affected
	 * @param Channel	UV channel, starting at 0
	 * @param Vertex1	Index of the first uv vertex that defines this face
	 * @param Vertex2	Index of the second uv vertex that defines this face
	 * @param Vertex3	Index of the third uv vertex that defines this face
	 */
	void SetFaceUV(
		int32 Index,
		int32 Channel,
		int32 Vertex1,
		int32 Vertex2,
		int32 Vertex3
	)
	{
		return GetDatasmithMesh().SetFaceUV(Index, Channel, Vertex1, Vertex2, Vertex3);
	}

	/**
	 * Gets the UV coordinates of the vertices of a face
	 *
	 * @param Index		Index of the face to retrieve the UVs from
	 * @param Channel	UV channel, starting at 0
	 * @param Vertex1	Index of the first uv vertex that defines this face
	 * @param Vertex2	Index of the second uv vertex that defines this face
	 * @param Vertex3	Index of the third uv vertex that defines this face
	 */
	void GetFaceUV(
		int32 Index,
		int32 Channel,
		int32& OutVertex1,
		int32& OutVertex2,
		int32& OutVertex3
	) const
	{
		GetDatasmithMesh().GetFaceUV(Index, Channel, OutVertex1, OutVertex2, OutVertex3);
	}

	/** Get the number of vertex color */
	int32 GetVertexColorCount() const
	{
		return GetDatasmithMesh().GetVertexColorCount();
	}

	/**
	 * Set a vertex color
	 *
	 * @param Index		Index of the vertex color that will be affected
	 * @param Color		The color for the vertex
	 */
	void SetVertexColor(
		int32 Index,
		uint8 R,
		uint8 G,
		uint8 B,
		uint8 A
	)
	{
		return GetDatasmithMesh().SetVertexColor( Index, FColor(R, G, B, A) );
	}

	/**
	 * Get the color for a vertex
	 *
	 * @param Index		Index of the vertex color to retrieve
	 */
	void GetVertexColor(
		int32 Index,
		uint8& OutR,
		uint8 OutG,
		uint8& OutB,
		uint8& OutA
	) const
	{
		FColor VertexColor(GetDatasmithMesh().GetVertexColor(Index));
		OutR = VertexColor.R;
		OutG = VertexColor.G;
		OutB = VertexColor.B;
		OutA = VertexColor.A;
	}

	/**
	 * Sets the UV channel that will be used as the source UVs for lightmap UVs generation at import, defaults to channel 0.
	 * Will be overwritten during Mesh export if we choose to generate the lightmap source UVs.
	 */
	void SetLightmapSourceUVChannel(
		int32 Channel
	)
	{
		return GetDatasmithMesh().SetLightmapSourceUVChannel(Channel);
	}

	/** Gets the UV channel that will be used for lightmap UVs generation at import */
	int32 GetLightmapSourceUVChannel() const
	{
		return GetDatasmithMesh().GetLightmapSourceUVChannel();
	}

	//--------------------------
	// LODs
	//--------------------------

	/** Adds a LOD mesh to this base LOD mesh */
	void AddLOD(const FDatasmithFacadeMesh& InLODMesh) { GetDatasmithMesh().AddLOD(InLODMesh.GetDatasmithMesh()); }
	int32 GetLODsCount() const { return GetDatasmithMesh().GetLODsCount(); }
	FDatasmithFacadeMesh* GetNewLOD(int32 Index)
	{
		if (FDatasmithMesh* MeshLOD = GetDatasmithMesh().GetLOD(Index))
		{
			return new FDatasmithFacadeMesh(RootOwnerMesh, *MeshLOD);
		}

		return nullptr;
	}

	//--------------------------
	// Misc
	//--------------------------
	/** Returns the total surface area */
	float ComputeArea() const
	{
		return GetDatasmithMesh().ComputeArea();
	}

	///////////////////////////////

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeMesh(
		const TSharedRef<FDatasmithMesh>& InRootOwnerMesh,
		FDatasmithMesh& InReferencedMesh
	)
		: RootOwnerMesh(InRootOwnerMesh)
		, InternalMesh(InReferencedMesh)
	{}

	FDatasmithMesh& GetDatasmithMesh() { return InternalMesh; }
	const FDatasmithMesh& GetDatasmithMesh() const { return InternalMesh; }

private:
	TSharedRef<FDatasmithMesh> RootOwnerMesh;
	FDatasmithMesh& InternalMesh;
};

class DATASMITHFACADE_API FDatasmithFacadeMeshElement : public FDatasmithFacadeElement
{
public:
	FDatasmithFacadeMeshElement(
		const TCHAR* InElementName
	);

	/** Get the output filename, it can be absolute or relative to the scene file */
	const TCHAR* GetFile() const
	{
		return GetDatasmithMeshElement()->GetFile();
	}

	/** Set the output filename, it can be absolute or relative to the scene file */
	void SetFile(
		const TCHAR* InFile
	)
	{
		GetDatasmithMeshElement()->SetFile(InFile);
	}

	/** Return a string representation of a MD5 hash of the content of the Mesh Element. Used in CalculateElementHash to quickly identify Element with identical content */
	void GetFileHash(
		TCHAR OutBuffer[33],
		size_t BufferSize
	) const;

	/** Set the MD5 hash of the current texture file. This should be a hash of its content. */
	void SetFileHash(
		const TCHAR* Hash
	);

	/**
	 * Set surface area and bounding box dimensions to be used on lightmap size calculation.
	 *
	 * @param InArea total surface area
	 * @param InWidth bounding box width
	 * @param InHeight bounding box height
	 * @param InDepth bounding box depth
	 */
	void SetDimensions(
		float InArea,
		float InWidth,
		float InHeight,
		float InDepth
	)
	{
		GetDatasmithMeshElement()->SetDimensions(InArea, InWidth, InHeight, InDepth);
	}

	/** Get the total surface area */
	float GetArea() const
	{
		return GetDatasmithMeshElement()->GetArea();
	}

	/** Get the bounding box width */
	float GetWidth() const
	{
		return GetDatasmithMeshElement()->GetWidth();
	}

	/** Get the bounding box height */
	float GetHeight() const
	{
		return GetDatasmithMeshElement()->GetHeight();
	}

	/** Get the bounding box depth */
	float GetDepth() const
	{
		return GetDatasmithMeshElement()->GetDepth();
	}

	/** Get the UV channel that will be used for the lightmap */
	int32 GetLightmapCoordinateIndex() const
	{
		return GetDatasmithMeshElement()->GetLightmapCoordinateIndex();
	}

	/**
	 * Set the UV channel that will be used for the lightmap
	 * Note: If the lightmap coordinate index is something greater than -1 it will make the importer skip the lightmap generation
	 */
	void SetLightmapCoordinateIndex(int32 UVChannel)
	{
		GetDatasmithMeshElement()->SetLightmapCoordinateIndex(UVChannel);
	}

	/** Get the source UV channel that will be used at import to generate the lightmap UVs */
	int32 GetLightmapSourceUV() const
	{
		return GetDatasmithMeshElement()->GetLightmapSourceUV();
	}

	/** Set the source UV channel that will be used at import to generate the lightmap UVs */
	void SetLightmapSourceUV(int32 UVChannel)
	{
		GetDatasmithMeshElement()->SetLightmapSourceUV(UVChannel);
	}

	/** Set the material slot Id to use the material MaterialPathName*/
	void SetMaterial(const TCHAR* MaterialPathName, int32 SlotId)
	{
		GetDatasmithMeshElement()->SetMaterial(MaterialPathName, SlotId);
	}

	/** Get the name of the material mapped to slot Id, return nullptr if slot isn't mapped */
	const TCHAR* GetMaterial(int32 SlotId) const
	{
		return GetDatasmithMeshElement()->GetMaterial(SlotId);
	}

	/** Get the number of material slot set on this mesh */
	int32 GetMaterialSlotCount() const
	{
		return GetDatasmithMeshElement()->GetMaterialSlotCount();
	}

	/** Get the material mapping for slot Index */
	FDatasmithFacadeMaterialID* GetMaterialSlotAt(
		int32 Index
	);

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeMeshElement(
		const TSharedRef<IDatasmithMeshElement>& InMeshElement
	);

	TSharedRef<IDatasmithMeshElement> GetDatasmithMeshElement() const
	{
		return StaticCastSharedRef<IDatasmithMeshElement>(InternalDatasmithElement);
	}
};