// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeElement.h"

class FDatasmithFacadeKeyValueProperty;
class FDatasmithFacadeScene;
class IDatasmithMetaDataElement;

class DATASMITHFACADE_API FDatasmithFacadeMetaData : public FDatasmithFacadeElement
{
public:	

	FDatasmithFacadeMetaData(
		const TCHAR* InElementName
	);

	// Add a property boolean property to the Datasmith actor.
	void AddPropertyBoolean(
		const TCHAR* InPropertyName,
		bool bInPropertyValue
	);

	// Add a property sRGBA color property to the Datasmith actor.
	void AddPropertyColor(
		const TCHAR*  InPropertyName,
		uint8 InR,
		uint8 InG,
		uint8 InB,
		uint8 InA
	);

	// Add a property float property to the Datasmith actor.
	void AddPropertyFloat(
		const TCHAR* InPropertyName,
		float InPropertyValue
	);

	// Add a property string property to the Datasmith actor.
	void AddPropertyString(
		const TCHAR* InPropertyName,
		const TCHAR* InPropertyValue
	);

	// Add a property texture property to the Datasmith actor.
	void AddPropertyTexture(
		const TCHAR* InPropertyName,
		const TCHAR* InTextureFilePath
	);

	// Add a property vector property to the Datasmith actor.
	void AddPropertyVector(
		const TCHAR* InPropertyName,
		const TCHAR* InPropertyValue
	);

	void AddProperty(
		const FDatasmithFacadeKeyValueProperty* InProperty
	);

	// Get the total amount of properties in this meta data.
	int32 GetPropertiesCount() const;

	/** Returns a new FDatasmithFacadeKeyValueProperty pointing to the KeyValueProperty at the given index, the returned value must be deleted after used, can be nullptr. */
	FDatasmithFacadeKeyValueProperty* GetNewProperty(
		int32 PropertyIndex
	) const;
	
	/** Sets the Datasmith element that is associated with this meta data */
	void SetAssociatedElement(
		const FDatasmithFacadeElement* Element
	);

	/** Remove the property from this meta data */
	void RemoveProperty(
		const FDatasmithFacadeKeyValueProperty* Property
	);

	/** Remove all properties in this meta data */
	void ResetProperties()
	{
		GetDatasmithMetaDataElement()->ResetProperties();
	}

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeMetaData(
		const TSharedRef<IDatasmithMetaDataElement>& InMetaDataElement
	);

	TSharedRef<IDatasmithMetaDataElement> GetDatasmithMetaDataElement() const;
};