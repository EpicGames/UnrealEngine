// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith SDK.
#include "DatasmithMeshExporter.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneExporter.h"

// Datasmith facade classes.
class FDatasmithFacadeActor;
class FDatasmithFacadeBaseMaterial;
class FDatasmithFacadeElement;
class FDatasmithFacadeMesh;
class FDatasmithFacadeMeshElement;
class FDatasmithFacadeMetaData;
class FDatasmithFacadeTexture;
class FDatasmithFacadeLevelVariantSets;
class FDatasmithFacadeLevelSequence;

class DATASMITHFACADE_API FDatasmithFacadeScene
{
public:

	// Copy from EDatasmithActorRemovalRule
	enum class EActorRemovalRule : uint32
	{
		/** Remove also the actors children */
		RemoveChildren,

		/** Keeps current relative transform as the relative transform to the new parent. */
		KeepChildrenAndKeepRelativeTransform,
	};

	FDatasmithFacadeScene(
		const TCHAR* InApplicationHostName,      // name of the host application used to build the scene
		const TCHAR* InApplicationVendorName,    // vendor name of the application used to build the scene
		const TCHAR* InApplicationProductName,   // product name of the application used to build the scene
		const TCHAR* InApplicationProductVersion // product version of the application used to build the scene
	);

	// Collect an element for the Datasmith scene to build.
	void AddActor(
		FDatasmithFacadeActor* InActorPtr // Datasmith scene element
	);

	int32 GetActorsCount() const;

	FDatasmithFacadeActor* GetNewActor(
		int32 ActorIndex
	);

	void RemoveActor(
		FDatasmithFacadeActor* InActorPtr,
		EActorRemovalRule RemovalRule = EActorRemovalRule::RemoveChildren
	);

	void RemoveActorAt(
		int32 ActorIndex,
		EActorRemovalRule RemovalRule = EActorRemovalRule::RemoveChildren
	)
	{
		SceneRef->RemoveActorAt(ActorIndex, static_cast<EDatasmithActorRemovalRule>(RemovalRule));
	}

	void AddMaterial(
		FDatasmithFacadeBaseMaterial* InMaterialPtr
	);

	/**
	 * Returns the number of material elements added to the scene.
	 */
	int32 GetMaterialsCount() const;

	/**
	 * Returns a new FDatasmithFacadeBaseMaterial pointing to the Material at the specified index.
	 * If the given index is invalid, the returned value is nullptr.
	 * 
	 * @param MaterialIndex The index of the material in the scene.
	 */
	FDatasmithFacadeBaseMaterial* GetNewMaterial(
		int32 MaterialIndex
	);

	/**
	 * Removes a Material Element from the scene.
	 *
	 * @param InMaterialPtr the Material Element to remove
	 */
	void RemoveMaterial(
		FDatasmithFacadeBaseMaterial* InMaterialPtr
	);

	void RemoveMaterialAt(
		int32 MaterialIndex
	)
	{
		SceneRef->RemoveMaterialAt(MaterialIndex);
	}

	FDatasmithFacadeMeshElement* ExportDatasmithMesh(
		FDatasmithFacadeMesh* Mesh,
		FDatasmithFacadeMesh* CollisionMesh = nullptr
	);

	bool ExportDatasmithMesh(
		FDatasmithFacadeMeshElement* MeshElement,
		FDatasmithFacadeMesh* Mesh,
		FDatasmithFacadeMesh* CollisionMesh = nullptr
	);

	void AddMesh(
		FDatasmithFacadeMeshElement* InMeshPtr
	);

	int32 GetMeshesCount() const
	{
		return SceneRef->GetMeshesCount();
	}

	FDatasmithFacadeMeshElement* GetNewMesh(
		int32 MeshIndex
	);

	void RemoveMesh(
		FDatasmithFacadeMeshElement* MeshElement
	);

	void RemoveMeshAt(
		int32 MeshIndex
	)
	{
		SceneRef->RemoveMeshAt(MeshIndex);
	}

	void AddTexture(
		FDatasmithFacadeTexture* InTexturePtr
	);

	int32 GetTexturesCount() const;

	/**
	 *	Returns a new FDatasmithFacadeTexture pointing to the Texture at the specified index.
	 *	If the given index is invalid, the returned value is nullptr.
	 */
	FDatasmithFacadeTexture* GetNewTexture(
		int32 TextureIndex
	);

	void RemoveTexture(
		FDatasmithFacadeTexture* InTexturePtr
	);

	void RemoveTextureAt(
		int32 TextureIndex
	)
	{
		SceneRef->RemoveTextureAt(TextureIndex);
	}

	void AddLevelVariantSets(
		FDatasmithFacadeLevelVariantSets* InLevelVariantSetsPtr
	);

	int32 GetLevelVariantSetsCount() const;

	FDatasmithFacadeLevelVariantSets* GetNewLevelVariantSets(
		int32 LevelVariantSetsIndex
	);

	void RemoveLevelVariantSets(
		FDatasmithFacadeLevelVariantSets* InLevelVariantSetsPtr
	);

	void RemoveLevelVariantSetsAt(
		int32 LevelVariantSetsIndex
	)
	{
		SceneRef->RemoveLevelVariantSetsAt(LevelVariantSetsIndex);
	}

	void AddLevelSequence(
		FDatasmithFacadeLevelSequence* InLevelSequence
	);

	int32 GetLevelSequencesCount() const;

	FDatasmithFacadeLevelSequence* GetNewLevelSequence(
		int32 LevelSequenceIndex
	);

	void RemoveLevelSequence(
		FDatasmithFacadeLevelSequence* InLevelSequence
	);

	void RemoveLevelSequenceAt(
		int32 LevelSequenceIndex
	)
	{
		SceneRef->RemoveLevelSequenceAt(LevelSequenceIndex);
	}

	void AddMetaData(
		FDatasmithFacadeMetaData* InMetaDataPtr
	);

	int32 GetMetaDataCount() const;

	/**
	 *	Returns a new FDatasmithFacadeMetaData pointing to the MetaData at the specified index.
	 *	If the given index is invalid, the returned value is nullptr.
	 */
	FDatasmithFacadeMetaData* GetNewMetaData(
		int32 MetaDataIndex
	);

	/**
	 *	Returns a new FDatasmithFacadeMetaData pointing to the MetaData associated to the specified DatasmithElement.
	 *	If there is no associated metadata or the element is null, the returned value is nullptr.
	 */
	FDatasmithFacadeMetaData* GetNewMetaData(
		FDatasmithFacadeElement* Element
	);

	void RemoveMetaData(
		FDatasmithFacadeMetaData* InMetaDataPtr
	);

	void RemoveMetaDataAt(
		int32 MetaDataIndex
	)
	{
		SceneRef->RemoveMetaDataAt(MetaDataIndex);
	}
	
	/** Set the Datasmith scene name */
	void SetName(const TCHAR* InName);

	/** Get the Datasmith scene file name */
	const TCHAR* GetName() const;

	/** Set the path to the folder where the .datasmith file will be saved. */
	void SetOutputPath(const TCHAR* InOutputPath);

	/** Get the path to the folder where the .datasmith file will be saved. */
	const TCHAR* GetOutputPath() const;

	/** Get the path were additional assets will be saved to. */
	const TCHAR* GetAssetsOutputPath() const;

	/** Get the Datasmith scene Geolocation data*/
	void GetGeolocation(double& OutLatitude, double& OutLongitude, double& OutElevation) const;

	/** Set the Datasmith scene Geolocation data*/
	void SetGeolocationLatitude(double Latitude);
	void SetGeolocationLongitude(double Longitude);
	void SetGeolocationElevation(double Elevation);

	/** Instantiate an exporter and register export start time */
	void PreExport();

	/** Validate assets and remove unused ones. */
	void CleanUp();

	/** 
	 * Manually shutdown the Facade and close down all the core engine systems and release the resources they allocated
	 * You don't need to call this function on Windows platform.
	 */
	static void Shutdown();

	/** Build and export a Datasmith scene instance and its scene element assets.
	 *  The passed InOutputPath parameter will override any Name and OutputPath previously specified.
	 *	@param bCleanupUnusedElements Remove unused meshes, textures and materials before exporting
	 *	@return True if the scene was properly exported.
	 */
	bool ExportScene(
		const TCHAR* InOutputPath, // Datasmith scene output file path
		bool bCleanupUnusedElements = true
	);

	/** Build and export a Datasmith scene instance and its scene element assets.
	 *	@param bCleanupUnusedElements Remove unused meshes, textures and materials before exporting
	 *	@return True if the scene was properly exported.
	 */
	bool ExportScene(
		bool bCleanupUnusedElements = true
	);

	/** Save Datasmith LevelSequences into files and specify those files in each corresponding LevelSequenceElement
	 */
	void SerializeLevelSequences();

	/**
	 * Set the Datasmith scene's label.
	 * This is mainly used in conjunction with DirectLink. The scene's label is used
	 * to name the source (stream) created to broadcast the content of the scene
	 */
	void SetLabel(
		const TCHAR* InSceneLabel
	);

	/** Return the Datasmith scene's label. */
	const TCHAR* GetLabel() const;

	void Reset()
	{
		SceneRef->Reset();
	}


	/** Gets the name of the host application which created the scene */
	const TCHAR* GetHost() const
	{
		return SceneRef->GetHost();
	}

	/**
	 * Sets the name of the host application from which we're exporting from.
	 *
	 * @param InHost	The host application name
	 */
	void SetHost(
		const TCHAR* InHost
	)
	{
		SceneRef->SetHost(InHost);
	}

	/** Returns the vendor name of the application used to export the scene */
	const TCHAR* GetVendor() const
	{
		return SceneRef->GetVendor();
	}

	/**
	 * Sets the vendor name of the application used to export the scene.
	 *
	 * @param InVendor	The application vendor name
	 */
	void SetVendor(
		const TCHAR* InApplicationVendorName
	)
	{
		SceneRef->SetVendor(InApplicationVendorName);
	}

	/** Returns the product name of the application used to export the scene */
	const TCHAR* GetProductName() const
	{
		return SceneRef->GetProductName();
	}

	/**
	 * Sets the product name of the application used to export the scene.
	 *
	 * @param InProductName	The application name
	 */
	void SetProductName(
		const TCHAR* InApplicationProductName
	)
	{
		SceneRef->SetProductName(InApplicationProductName);
	}

	/** Returns the product version of the application used to export the scene */
	const TCHAR* GetProductVersion() const
	{
		return SceneRef->GetProductVersion();
	}

	/**
	 * Sets the product version of the application used to export the scene.
	 *
	 * @param InProductVersion	The application version
	 */
	void SetProductVersion(
		const TCHAR* InApplicationProductVersion
	)
	{
		SceneRef->SetProductVersion(InApplicationProductVersion);
	}

#ifdef SWIG_FACADE
protected:
#endif
	
	// Return the build Datasmith scene instance.
	TSharedRef<IDatasmithScene> GetScene() const;

private:

	// Datasmith scene instance built with the collected elements.
	TSharedRef<IDatasmithScene> SceneRef;

	// Datasmith scene exporter
	TSharedRef<FDatasmithSceneExporter> SceneExporterRef;
};
