// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeElement.h"


class DATASMITHFACADE_API FDatasmithFacadeTexture : public FDatasmithFacadeElement
{
public:
	/**
	 *	Possible Datasmith texture modes.
	 *	Copy of EDatasmithTextureMode from DatasmithCore DatasmithDefinitions.h.
	 */
	enum class ETextureMode : uint8
	{
		Diffuse,
		Specular,
		Normal,
		NormalGreenInv,
		Displace,
		Other,
		Bump
	};

	/**
	 *	Texture filtering for textures. 
	 *	Copy of EDatasmithTextureFilter from DatasmithCore DatasmithDefinitions.h.
	 */
	enum class ETextureFilter
	{
		Nearest,
		Bilinear,
		Trilinear,
		/** Use setting from the Texture Group. */
		Default
	};

	/**
	 *	Texture address mode for textures.  Note: Preserve enum order.
	 *	Copy of EDatasmithTextureAddress from DatasmithCore DatasmithDefinitions.h.
	 */
	enum class ETextureAddress
	{
		Wrap,
		Clamp,
		Mirror
	};

	/**
	 *	Texture format for raw data importing.
	 *	Copy of EDatasmithTextureFormat from DatasmithCore DatasmithDefinitions.h.
	 */
	enum class ETextureFormat
	{
		PNG,
		JPEG
	};

	/**
	 *	Texture color space.
	 *	Default: Leave at whatever is default for the texture mode
	 *	sRGB: Enable the sRGB boolean regardless of texture mode
	 *	Linear: Disable the sRGB boolean regardless of texture mode
	 *	Copy of EDatasmithColorSpace from DatasmithCore DatasmithDefinitions.h.
	 */
	enum class EColorSpace
	{
		Default,
		sRGB,
		Linear,
	};

public:

	FDatasmithFacadeTexture(
		const TCHAR* InElementName // Datasmith element name
	);

	virtual ~FDatasmithFacadeTexture() {}

	/** Get texture filename */
	const TCHAR* GetFile() const;

	/** Set texture filename */
	void SetFile( const TCHAR* File );

	/** Set the output data buffer, used only when no output filename is set
	 *
	 * @param InData data to load the texture from
	 * @param InDataSize size in bytes of the buffer
	 * @param InFormat texture format(e.g. png or jpeg)
	 *
	 * @note The given data is not freed by the DatasmithImporter.
	 */
	void SetData( const uint8* InData, uint32 InDataSize, ETextureFormat InFormat );

	/** Return the optional data, if loading from memory. Must be callable from any thread. */
	const uint8* GetData( uint32& OutDataSize, ETextureFormat& OutFormat ) const;

	/** Return a string representation of a MD5 hash of the content of the Texture Element. Used in CalculateElementHash to quickly identify Element with identical content */
	void GetFileHash( TCHAR OutBuffer[33], size_t BufferSize ) const;

	/** Set the MD5 hash of the current texture file. This should be a hash of its content. */
	void SetFileHash( const TCHAR* Hash );

	/** Get texture usage */
	ETextureMode GetTextureMode() const;

	/** Set texture usage */
	void SetTextureMode( ETextureMode Mode );

	/** Get texture filter */
	ETextureFilter GetTextureFilter() const;

	/** Set texture filter */
	void SetTextureFilter( ETextureFilter Filter );

	/** Get texture X axis address mode */
	ETextureAddress GetTextureAddressX() const;

	/** Set texture X axis address mode */
	void SetTextureAddressX( ETextureAddress Mode );

	/** Get texture Y axis address mode */
	ETextureAddress GetTextureAddressY() const;

	/** Set texture Y axis address mode */
	void SetTextureAddressY( ETextureAddress Mode );

	/** Get allow texture resizing */
	bool GetAllowResize() const;

	/** Set allow texture resizing */
	void SetAllowResize( bool bAllowResize );

	/** Get texture gamma <= 0 for auto */
	float GetRGBCurve() const;

	/** Set texture gamma <= 0 for auto */
	void SetRGBCurve( const float InRGBCurve );

	/** Gets the color space of the texture */
	EColorSpace GetSRGB() const;

	/** Sets the color space of the texture */
	void SetSRGB( EColorSpace Option );

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeTexture( TSharedRef<IDatasmithTextureElement> InTextureElement );

	TSharedRef<IDatasmithTextureElement> GetDatasmithTextureElement() const;
};