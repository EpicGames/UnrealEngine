// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeElement.h"
#include "DatasmithFacadeMaterial.h"

#include "CoreMinimal.h"

class IDatasmithExpressionInput;
class IDatasmithExpressionParameter;
class FDatasmithFacadeKeyValueProperty;
class IDatasmithMaterialExpression;
class IDatasmithUEPbrMaterialElement;

class FDatasmithFacadeMaterialExpression;

class DATASMITHFACADE_API FDatasmithFacadeExpressionInput
{
public:
	const TCHAR* GetName() const;
	void SetName(const TCHAR* InName);

	/** Returns a new FDatasmithFacadeMaterialExpression pointing to the connected expression, the returned value must be deleted after used, can be nullptr. */
	FDatasmithFacadeMaterialExpression* GetNewFacadeExpression();
	void SetExpression( FDatasmithFacadeMaterialExpression* InExpression );

	int32 GetOutputIndex() const;
	void SetOutputIndex( int32 InOutputIndex );

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeExpressionInput( IDatasmithExpressionInput* InExpressionInput, const TSharedPtr<IDatasmithUEPbrMaterialElement>& InMaterialElement )
		: InternalExpressionInput(InExpressionInput)
		, ReferencedMaterial(InMaterialElement)
	{}

	IDatasmithExpressionInput& GetExpressionInput() { return *InternalExpressionInput; }
	const IDatasmithExpressionInput& GetExpressionInput() const { return *InternalExpressionInput; }

private:

	IDatasmithExpressionInput* InternalExpressionInput;

	//We hold a shared pointer to the material to make sure it stays valid while a facade objects points to it.
	TSharedPtr<IDatasmithUEPbrMaterialElement> ReferencedMaterial;
};

enum class EDatasmithFacadeMaterialExpressionType : uint64
{
	ConstantBool,
	ConstantColor,
	ConstantScalar,
	FlattenNormal,
	FunctionCall,
	Generic,
	Texture,
	TextureCoordinate,
	None = 255,
};

// not trivial to reuse EDatasmithMaterialExpressionType (eg with a "using" declaration). #swig
#define DS_CHECK_ENUM_MISMATCH(name) static_assert((uint64)EDatasmithFacadeMaterialExpressionType::name == (uint64)EDatasmithMaterialExpressionType::name, "enum mismatch");
DS_CHECK_ENUM_MISMATCH(None)
DS_CHECK_ENUM_MISMATCH(ConstantBool)
DS_CHECK_ENUM_MISMATCH(ConstantColor)
DS_CHECK_ENUM_MISMATCH(ConstantScalar)
DS_CHECK_ENUM_MISMATCH(FlattenNormal)
DS_CHECK_ENUM_MISMATCH(FunctionCall)
DS_CHECK_ENUM_MISMATCH(Generic)
DS_CHECK_ENUM_MISMATCH(Texture)
DS_CHECK_ENUM_MISMATCH(TextureCoordinate)
#undef DS_CHECK_ENUM_MISMATCH

class DATASMITHFACADE_API FDatasmithFacadeMaterialExpression
{
public:

	/** The name of the expression. Used as parameter name for material instances. */
	const TCHAR* GetName() const;
	void SetName( const TCHAR* InName );

	EDatasmithFacadeMaterialExpressionType GetExpressionType() const;

	/** Connects the default output to an expression input */
	void ConnectExpression( FDatasmithFacadeExpressionInput& ExpressionInput );

	/** Connects a specific output to an expression input */
	void ConnectExpression( FDatasmithFacadeExpressionInput& ExpressionInput, int32 OutputIndex );

	int32 GetInputCount() const;
	/**
	 * Returns a pointer to a new FDatasmithFacadeExpressionInput object wrapping the IDatasmithExpressionInput at the specified index if it exists.
	 * Returns null if the index is invalid.
	 * The caller is responsible for the destruction of returned object.
	 */
	FDatasmithFacadeExpressionInput* GetNewFacadeInput( int32 Index );

	/** The output index to use by default for this expression when connecting it to other inputs. */
	int32 GetDefaultOutputIndex() const;
	void SetDefaultOutputIndex( int32 OutputIndex );

	virtual ~FDatasmithFacadeMaterialExpression() = default;

	void ResetExpression() { GetMaterialExpression()->ResetExpression(); }

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeMaterialExpression( IDatasmithMaterialExpression* InMaterialExpression, const TSharedPtr<IDatasmithUEPbrMaterialElement>& InMaterialElement )
		: InternalMaterialExpression( InMaterialExpression )
		, ReferencedMaterial( InMaterialElement )
	{}

	IDatasmithMaterialExpression* GetMaterialExpression() { return InternalMaterialExpression; }
	const IDatasmithMaterialExpression* GetMaterialExpression() const { return InternalMaterialExpression; }

	IDatasmithMaterialExpression* InternalMaterialExpression;

	//We hold a shared pointer to the material to make sure it stays valid while a facade objects points to it.
	TSharedPtr<IDatasmithUEPbrMaterialElement> ReferencedMaterial;
};

/**
 * Facade representation of Abstract class IDatasmithExpressionParameter
 */
class FDatasmithFacadeExpressionParameter
{
public:
	virtual ~FDatasmithFacadeExpressionParameter() = default;

	virtual const TCHAR* GetGroupName() const = 0;
	virtual void SetGroupName( const TCHAR* InGroupName ) = 0;
};

/**
 * Represents a UMaterialExpressionStaticBoolParameter
 */
class FDatasmithFacadeMaterialExpressionBool : public FDatasmithFacadeMaterialExpression, public FDatasmithFacadeExpressionParameter
{
public:
	bool GetBool() const;
	void SetBool(bool InValue);

	virtual const TCHAR* GetGroupName() const override;
	virtual void SetGroupName( const TCHAR* InGroupName ) override;

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeMaterialExpressionBool( IDatasmithMaterialExpression* InMaterialExpression, const TSharedPtr<IDatasmithUEPbrMaterialElement>& InMaterialElement )
		: FDatasmithFacadeMaterialExpression( InMaterialExpression, InMaterialElement )
	{}
};

class FDatasmithFacadeMaterialExpressionColor : public FDatasmithFacadeMaterialExpression, public FDatasmithFacadeExpressionParameter
{
public:
	/**
	 * Get the color value of the expression in a sRGB format
	 */
	void GetsRGBColor( uint8& OutR, uint8& OutG, uint8& OutB, uint8& OutA ) const;
	/**
	 * Set the color value of the expression in a sRGB format
	 */
	void SetsRGBColor( uint8 R, uint8 G, uint8 B, uint8 A );

	/**
	 * Get the color value of the expression in a linear format with values between 0 and 1.
	 */
	void GetColor( float& OutR, float& OutG, float& OutB, float& OutA ) const;
	/**
	 * Set the color value of the expression in a linear format with values between 0 and 1.
	 */
	void SetColor( float R, float G, float B, float A );

	virtual const TCHAR* GetGroupName() const override;
	virtual void SetGroupName( const TCHAR* InGroupName ) override;

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeMaterialExpressionColor( IDatasmithMaterialExpression* InMaterialExpression, const TSharedPtr<IDatasmithUEPbrMaterialElement>& InMaterialElement )
		: FDatasmithFacadeMaterialExpression( InMaterialExpression, InMaterialElement )
	{}
};

class FDatasmithFacadeMaterialExpressionScalar : public FDatasmithFacadeMaterialExpression, public FDatasmithFacadeExpressionParameter
{
public:
	float GetScalar() const;
	void SetScalar( float InScalar );

	virtual const TCHAR* GetGroupName() const override;
	virtual void SetGroupName( const TCHAR* InGroupName ) override;

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeMaterialExpressionScalar( IDatasmithMaterialExpression* InMaterialExpression, const TSharedPtr<IDatasmithUEPbrMaterialElement>& InMaterialElement )
		: FDatasmithFacadeMaterialExpression( InMaterialExpression, InMaterialElement )
	{}
};

class FDatasmithFacadeMaterialExpressionTexture : public FDatasmithFacadeMaterialExpression, public FDatasmithFacadeExpressionParameter
{
public:
	const TCHAR* GetTexturePathName() const;
	void SetTexturePathName( const TCHAR* InTexturePathName );

	/**
	 * Inputs
	 */
	FDatasmithFacadeExpressionInput GetInputCoordinate();

	/**
	 * Outputs:
	 * - RGB
	 * - R
	 * - G
	 * - B
	 * - A
	 */

	virtual const TCHAR* GetGroupName() const override;
	virtual void SetGroupName( const TCHAR* InGroupName ) override;

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeMaterialExpressionTexture( IDatasmithMaterialExpression* InMaterialExpression, const TSharedPtr<IDatasmithUEPbrMaterialElement>& InMaterialElement )
		: FDatasmithFacadeMaterialExpression( InMaterialExpression, InMaterialElement )
	{}
};

class FDatasmithFacadeMaterialExpressionTextureCoordinate : public FDatasmithFacadeMaterialExpression
{
public:
	int32 GetCoordinateIndex() const;
	void SetCoordinateIndex( int32 InCoordinateIndex );

	float GetUTiling() const;
	void SetUTiling( float InUTiling );

	float GetVTiling() const;
	void SetVTiling( float InVTiling );

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeMaterialExpressionTextureCoordinate( IDatasmithMaterialExpression* InMaterialExpression, const TSharedPtr<IDatasmithUEPbrMaterialElement>& InMaterialElement )
		: FDatasmithFacadeMaterialExpression( InMaterialExpression, InMaterialElement )
	{}
};

class FDatasmithFacadeMaterialExpressionFlattenNormal : public FDatasmithFacadeMaterialExpression
{
public:
	/**
	 * Inputs
	 */
	FDatasmithFacadeExpressionInput GetNormal() const;

	FDatasmithFacadeExpressionInput GetFlatness() const;

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeMaterialExpressionFlattenNormal( IDatasmithMaterialExpression* InMaterialExpression, const TSharedPtr<IDatasmithUEPbrMaterialElement>& InMaterialElement )
		: FDatasmithFacadeMaterialExpression( InMaterialExpression, InMaterialElement )
	{}
};

class FDatasmithFacadeMaterialExpressionGeneric : public FDatasmithFacadeMaterialExpression
{
public:
	void SetExpressionName( const TCHAR* InExpressionName );
	const TCHAR* GetExpressionName() const;

	/** Get the total amount of properties in this expression */
	int32 GetPropertiesCount() const;

	/** Add a property to this expression*/
	void AddProperty( const FDatasmithFacadeKeyValueProperty* InPropertyPtr );

	/** Returns a new FDatasmithFacadeKeyValueProperty pointing to the property at the given index, the returned value must be deleted after used, can be nullptr. */
	FDatasmithFacadeKeyValueProperty* GetNewProperty( int32 Index );

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeMaterialExpressionGeneric( IDatasmithMaterialExpression* InMaterialExpression, const TSharedPtr<IDatasmithUEPbrMaterialElement>& InMaterialElement )
		: FDatasmithFacadeMaterialExpression( InMaterialExpression, InMaterialElement )
	{}
};

class FDatasmithFacadeMaterialExpressionFunctionCall : public FDatasmithFacadeMaterialExpression
{
public:
	void SetFunctionPathName( const TCHAR* InFunctionPathName );
	const TCHAR* GetFunctionPathName() const;

#ifdef SWIG_FACADE
protected:
#endif
	FDatasmithFacadeMaterialExpressionFunctionCall( IDatasmithMaterialExpression* InMaterialExpression, const TSharedPtr<IDatasmithUEPbrMaterialElement>& InMaterialElement )
		: FDatasmithFacadeMaterialExpression( InMaterialExpression, InMaterialElement )
	{}
};


class DATASMITHFACADE_API FDatasmithFacadeUEPbrMaterial :
	public FDatasmithFacadeBaseMaterial
{
	friend class FDatasmithFacadeScene;

public:
	FDatasmithFacadeUEPbrMaterial( const TCHAR* InElementName );

	virtual ~FDatasmithFacadeUEPbrMaterial() {}

	FDatasmithFacadeExpressionInput GetBaseColor() const;
	FDatasmithFacadeExpressionInput GetMetallic() const;
	FDatasmithFacadeExpressionInput GetSpecular() const;
	FDatasmithFacadeExpressionInput GetRoughness() const;
	FDatasmithFacadeExpressionInput GetEmissiveColor() const;
	FDatasmithFacadeExpressionInput GetOpacity() const;
	FDatasmithFacadeExpressionInput GetNormal() const;
	FDatasmithFacadeExpressionInput GetRefraction() const;
	FDatasmithFacadeExpressionInput GetAmbientOcclusion() const;
	FDatasmithFacadeExpressionInput GetClearCoat() const;
	FDatasmithFacadeExpressionInput GetClearCoatRoughness() const;
	FDatasmithFacadeExpressionInput GetWorldPositionOffset() const;
	FDatasmithFacadeExpressionInput GetMaterialAttributes() const;

	int GetBlendMode() const;
	void SetBlendMode( int bInBlendMode );

	bool GetTwoSided() const;
	void SetTwoSided( bool bTwoSided );

	bool GetUseMaterialAttributes() const;
	void SetUseMaterialAttributes( bool bInUseMaterialAttributes );

	/** If a material is only referenced by other materials then it is only used as a material function and there is no need to instantiate it. */
	bool GetMaterialFunctionOnly() const;
	void SetMaterialFunctionOnly(bool bInMaterialFunctionOnly);

	float GetOpacityMaskClipValue() const;
	void SetOpacityMaskClipValue(float InClipValue);

	int32 GetExpressionsCount() const;

	/** Returns a new FDatasmithFacadeMaterialExpression pointing to the expression at the given index, the returned value must be deleted after used, can be nullptr. */
	FDatasmithFacadeMaterialExpression* GetNewFacadeExpression( int32 Index );

	int32 GetExpressionIndex( const FDatasmithFacadeMaterialExpression& Expression ) const;

	template< typename T >
	T AddMaterialExpression()
	{
	}

	/** Reset all expression to their default values and remove all connections */
	void ResetExpressionGraph()
	{
		constexpr bool bRemoveAllExpressions = false;
		GetDatasmithUEPbrMaterialElement()->ResetExpressionGraph( bRemoveAllExpressions );
	}

	/** If a parent material is generated from this material, this will be its label. If none, the instance and the parent will have the same label. */
	void SetParentLabel( const TCHAR* InParentLabel );
	const TCHAR* GetParentLabel() const;

#ifdef SWIG_FACADE
protected:
#endif

	FDatasmithFacadeUEPbrMaterial( const TSharedRef<IDatasmithUEPbrMaterialElement>& InMaterialRef );

	IDatasmithMaterialExpression* AddMaterialExpression( const EDatasmithFacadeMaterialExpressionType ExpressionType );

	TSharedRef<IDatasmithUEPbrMaterialElement> GetDatasmithUEPbrMaterialElement() const;
};

template<>
inline FDatasmithFacadeMaterialExpressionBool FDatasmithFacadeUEPbrMaterial::AddMaterialExpression< FDatasmithFacadeMaterialExpressionBool >()
{
	return FDatasmithFacadeMaterialExpressionBool( AddMaterialExpression( EDatasmithFacadeMaterialExpressionType::ConstantBool), GetDatasmithUEPbrMaterialElement() );
}

template<>
inline FDatasmithFacadeMaterialExpressionColor FDatasmithFacadeUEPbrMaterial::AddMaterialExpression< FDatasmithFacadeMaterialExpressionColor >()
{
	return FDatasmithFacadeMaterialExpressionColor( AddMaterialExpression( EDatasmithFacadeMaterialExpressionType::ConstantColor ), GetDatasmithUEPbrMaterialElement() );
}

template<>
inline FDatasmithFacadeMaterialExpressionFlattenNormal FDatasmithFacadeUEPbrMaterial::AddMaterialExpression< FDatasmithFacadeMaterialExpressionFlattenNormal >()
{
	return FDatasmithFacadeMaterialExpressionFlattenNormal( AddMaterialExpression( EDatasmithFacadeMaterialExpressionType::FlattenNormal ), GetDatasmithUEPbrMaterialElement() );
}

template<>
inline FDatasmithFacadeMaterialExpressionFunctionCall FDatasmithFacadeUEPbrMaterial::AddMaterialExpression< FDatasmithFacadeMaterialExpressionFunctionCall >()
{
	return FDatasmithFacadeMaterialExpressionFunctionCall( AddMaterialExpression( EDatasmithFacadeMaterialExpressionType::FunctionCall ), GetDatasmithUEPbrMaterialElement() );
}

template<>
inline FDatasmithFacadeMaterialExpressionGeneric FDatasmithFacadeUEPbrMaterial::AddMaterialExpression< FDatasmithFacadeMaterialExpressionGeneric >()
{
	return FDatasmithFacadeMaterialExpressionGeneric( AddMaterialExpression( EDatasmithFacadeMaterialExpressionType::Generic ), GetDatasmithUEPbrMaterialElement() );
}

template<>
inline FDatasmithFacadeMaterialExpressionScalar FDatasmithFacadeUEPbrMaterial::AddMaterialExpression< FDatasmithFacadeMaterialExpressionScalar >()
{
	return FDatasmithFacadeMaterialExpressionScalar( AddMaterialExpression( EDatasmithFacadeMaterialExpressionType::ConstantScalar ), GetDatasmithUEPbrMaterialElement() );
}

template<>
inline FDatasmithFacadeMaterialExpressionTexture FDatasmithFacadeUEPbrMaterial::AddMaterialExpression< FDatasmithFacadeMaterialExpressionTexture >()
{
	return FDatasmithFacadeMaterialExpressionTexture( AddMaterialExpression( EDatasmithFacadeMaterialExpressionType::Texture ), GetDatasmithUEPbrMaterialElement() );
}

template<>
inline FDatasmithFacadeMaterialExpressionTextureCoordinate FDatasmithFacadeUEPbrMaterial::AddMaterialExpression< FDatasmithFacadeMaterialExpressionTextureCoordinate >()
{
	return FDatasmithFacadeMaterialExpressionTextureCoordinate( AddMaterialExpression( EDatasmithFacadeMaterialExpressionType::TextureCoordinate ), GetDatasmithUEPbrMaterialElement() );
}
