// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FDatasmithUniqueNameProvider;

class FDatasmithFacadeUniqueNameProvider
{
public:
	DATASMITHFACADE_API FDatasmithFacadeUniqueNameProvider();
	
	/**
	 * Generates a unique name
	 * @param BaseName Name that will be suffixed with an index to be unique
	 * @return TCHAR* Unique name. Calling "Contains()" with this name will be false. 
	 *                Pointer is only valid until the next GenerateUniqueName() call.
	 */
	DATASMITHFACADE_API const TCHAR* GenerateUniqueName(const TCHAR* BaseName);

	/**
	 * Reserves space in the internal data structures to contain at least the number of name specified.
	 * @param NumberOfName The number of name to reserve for.
	 */
	DATASMITHFACADE_API void Reserve( int32 NumberOfName );

	/**
	 * Register a name as known
	 * @param Name name to register
	 */
	DATASMITHFACADE_API void AddExistingName(const TCHAR* Name);
	
	/**
	 * Remove a name from the list of existing name
	 * @param Name name to unregister
	 */
	DATASMITHFACADE_API void RemoveExistingName(const TCHAR* Name);

private:
	TUniquePtr<FDatasmithUniqueNameProvider> InternalNameProvider;
	FString CachedGeneratedName;
};
