// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Datasmith facade.
#include "DatasmithFacadeElement.h"
#include "DatasmithFacadeActor.h"

// Datasmith SDK classes.
class IDatasmithBasePropertyCaptureElement;
class IDatasmithActorBindingElement;
class IDatasmithVariantElement;
class IDatasmithVariantSetElement;
class IDatasmithLevelVariantSetsElement;

// Datasmith facade classes.
class FDatasmithFacadeBaseMaterial;

enum class EDatasmithFacadePropertyCategory : uint64
{
	Undefined = 0,
	Generic = 1,
	RelativeLocation = 2,
	RelativeRotation = 4,
	RelativeScale3D = 8,
	Visibility = 16,
	Material = 32,
	Color = 64,
	Option = 128
};

// not trivial to reuse EDatasmithMaterialExpressionType (eg with a "using" declaration). #swig
#define DS_CHECK_ENUM_MISMATCH(name) static_assert((uint64)EDatasmithFacadePropertyCategory::name == (uint64)EDatasmithPropertyCategory::name, "enum mismatch");
DS_CHECK_ENUM_MISMATCH(Undefined)
DS_CHECK_ENUM_MISMATCH(Generic)
DS_CHECK_ENUM_MISMATCH(RelativeLocation)
DS_CHECK_ENUM_MISMATCH(RelativeRotation)
DS_CHECK_ENUM_MISMATCH(RelativeScale3D)
DS_CHECK_ENUM_MISMATCH(Visibility)
DS_CHECK_ENUM_MISMATCH(Material)
DS_CHECK_ENUM_MISMATCH(Color)
DS_CHECK_ENUM_MISMATCH(Option)
#undef DS_CHECK_ENUM_MISMATCH

class DATASMITHFACADE_API FDatasmithFacadePropertyCapture :
	public FDatasmithFacadeElement
{
public:

	FDatasmithFacadePropertyCapture();

	virtual ~FDatasmithFacadePropertyCapture() {}

	void SetPropertyPath(const TCHAR* Path);

	const TCHAR* GetPropertyPath();

	EDatasmithFacadePropertyCategory GetCategory() const;

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadePropertyCapture(
		const TSharedRef<IDatasmithBasePropertyCaptureElement>& InInternalPropertyCapture
	);

	TSharedRef<IDatasmithBasePropertyCaptureElement> GetDatasmithPropertyCapture() const;
};

class DATASMITHFACADE_API FDatasmithFacadeActorBinding :
	public FDatasmithFacadeElement
{
public:

	FDatasmithFacadeActorBinding(
		FDatasmithFacadeActor* InActorPtr
	);

	virtual ~FDatasmithFacadeActorBinding() {}

	void AddPropertyCapture(
		FDatasmithFacadePropertyCapture* InPropertyCapturePtr
	);

	int32 GetPropertyCapturesCount() const;

	FDatasmithFacadePropertyCapture* GetNewPropertyCapture(
		int32 PropertyCaptureIndex
	);

	void RemovePropertyCapture(
		FDatasmithFacadePropertyCapture* InPropertyCapturePtr
	);

	void AddRelativeLocationCapture(
		double X,
		double Y,
		double Z
	);

	void AddRelativeRotationCapture(
		double Pitch,
		double Yaw,
		double Roll
	);

	void AddRelativeRotationCapture(
		double X,
		double Y,
		double Z,
		double W
	);

	void AddRelativeScaleCapture(
		double X,
		double Y,
		double Z
	);

	void AddRelativeTransformCapture(
		const double InMatrix[16],
		bool bRowMajor = false
	);

	void AddRelativeTransformCapture(
		const float InMatrix[16],
		bool bRowMajor = false
	)
	{
		double InMatrixD[16];
		for(int I = 0; I < 16; ++I)
		{
			InMatrixD[I] = InMatrix[I];
		}
		AddRelativeTransformCapture(InMatrixD, bRowMajor);
	}

	void AddVisibilityCapture(
		bool bInVisibility
	);

	void AddMaterialCapture(
		FDatasmithFacadeBaseMaterial* InMaterialPtr
	);

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadeActorBinding(
		const TSharedRef<IDatasmithActorBindingElement>& InInternalActorBinding
	);

	TSharedRef<IDatasmithActorBindingElement> GetDatasmithActorBinding() const;
};

class DATASMITHFACADE_API FDatasmithFacadeVariant :
	public FDatasmithFacadeElement
{
public:

	FDatasmithFacadeVariant(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadeVariant() {}

	void AddActorBinding(
		FDatasmithFacadeActorBinding* InActorBindingPtr
	);

	int32 GetActorBindingsCount() const;

	FDatasmithFacadeActorBinding* GetNewActorBinding(
		int32 ActorBindingIndex
	);

	void RemoveActorBinding(
		FDatasmithFacadeActorBinding* InActorBindingPtr
	);

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadeVariant(
		const TSharedRef<IDatasmithVariantElement>& InInternalVariant
	);

	TSharedRef<IDatasmithVariantElement> GetDatasmithVariant() const;
};

class DATASMITHFACADE_API FDatasmithFacadeVariantSet :
	public FDatasmithFacadeElement
{
public:

	FDatasmithFacadeVariantSet(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadeVariantSet() {}

	void AddVariant(
		FDatasmithFacadeVariant* InVariantPtr
	);

	int32 GetVariantsCount() const;

	FDatasmithFacadeVariant* GetNewVariant(
		int32 VariantIndex
	);

	void RemoveVariant(
		FDatasmithFacadeVariant* InVariantPtr
	);

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadeVariantSet(
		const TSharedRef<IDatasmithVariantSetElement>& InInternalVariantSet
	);

	TSharedRef<IDatasmithVariantSetElement> GetDatasmithVariantSet() const;
};

class DATASMITHFACADE_API FDatasmithFacadeLevelVariantSets :
	public FDatasmithFacadeElement
{
public:

	FDatasmithFacadeLevelVariantSets(
		const TCHAR* InElementName
	);

	virtual ~FDatasmithFacadeLevelVariantSets() {}

	void AddVariantSet(
		FDatasmithFacadeVariantSet* InVariantSetPtr
	);

	int32 GetVariantSetsCount() const;

	FDatasmithFacadeVariantSet* GetNewVariantSet(
		int32 VariantSetIndex
	);

	void RemoveVariantSet(
		FDatasmithFacadeVariantSet* InVariantSetPtr
	);

#ifdef SWIG_FACADE
protected:
#endif

	explicit FDatasmithFacadeLevelVariantSets(
		const TSharedRef<IDatasmithLevelVariantSetsElement>& InInternalLevelVariantsSet
	);

	TSharedRef<IDatasmithLevelVariantSetsElement> GetDatasmithLevelVariantSets() const;
};
