// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithDirectLink.h"

#include "CoreTypes.h"

class FDatasmithFacadeScene;
class FDatasmithFacadeEndpointObserver;

class DATASMITHFACADE_API FDatasmithFacadeDirectLink
{
public:
	static bool Init();
	static bool Init(bool bUseDatasmithExporterUI, const TCHAR* RemoteEngineDirPath);
	static int ValidateCommunicationSetup() { return FDatasmithDirectLink::ValidateCommunicationSetup(); }
	static bool Shutdown();

	bool InitializeForScene(FDatasmithFacadeScene* FacadeScene);
	bool UpdateScene(FDatasmithFacadeScene* FacadeScene);

	/**
	 * Close the initialized DirectLink source if any.
	 */
	void CloseCurrentSource();

	/**
	 * Register an IEndpointObserver that will be notified periodically with the last state of the swarm.
	 * @param Observer      Object that should be notified
	 */
	void AddEndpointObserver(FDatasmithFacadeEndpointObserver* Observer);

	/**
	 * Removes a previously added observer
	 * @param Observer      Observer to remove
	 */
	void RemoveEndpointObserver(FDatasmithFacadeEndpointObserver* Observer);

private:
	FDatasmithDirectLink Impl;
};