// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DirectLinkEndpoint.h"

#include "CoreTypes.h"

class FDatasmithFacadeScene;


struct DATASMITHFACADE_API FDatasmithFacadeRawInfo
{
	struct DATASMITHFACADE_API FDatasmithFacadeDataPointId
	{
	public:

		const TCHAR* GetName() const { return *DataPointId.Name; }
		FGuid GetId() const { return DataPointId.Id; }
		bool IsPublic() const { return DataPointId.bIsPublic; }

#ifdef SWIG_FACADE
	private:
#endif
		explicit FDatasmithFacadeDataPointId(const DirectLink::FRawInfo::FDataPointId& InDataPointId)
			: DataPointId(InDataPointId)
		{}

	private:
		DirectLink::FRawInfo::FDataPointId DataPointId;
	};

	struct DATASMITHFACADE_API FDatasmithFacadeEndpointInfo
	{
	public:
		const TCHAR* GetName() const { return *EndpointInfo.Name; }
		
		int32 GetNumberOfDestinations() const { return EndpointInfo.Destinations.Num(); }
		/**
		 * Returns the Destination info at the given index, returns null if the index is invalid.
		 * The caller is responsible of deleting the returned object pointer.
		 */
		FDatasmithFacadeDataPointId* GetNewDestination(int32 Index) const;
		
		int32 GetNumberOfSources() const { return EndpointInfo.Sources.Num(); }
		/**
		 * Returns the Source info at the given index, returns null if the index is invalid.
		 * The caller is responsible of deleting the returned object pointer.
		 */
		FDatasmithFacadeDataPointId* GetNewSource(int32 Index) const;

		const TCHAR* GetUserName() const { return *EndpointInfo.UserName; }
		const TCHAR* GetExecutableName() const { return *EndpointInfo.ExecutableName; }
		const TCHAR* GetComputerName() const { return *EndpointInfo.ComputerName; }
		bool IsLocal() const { return EndpointInfo.bIsLocal; }
		uint32 GetProcessId() const { return EndpointInfo.ProcessId; }

#ifdef SWIG_FACADE
	private:
#endif
		explicit FDatasmithFacadeEndpointInfo(const DirectLink::FRawInfo::FEndpointInfo& InEndpointInfo)
			: EndpointInfo(InEndpointInfo)
		{}

	private:
		DirectLink::FRawInfo::FEndpointInfo EndpointInfo;
	};

	struct DATASMITHFACADE_API FDatasmithFacadeDataPointInfo
	{
		FMessageAddress GetEndpointAddress() const { return DataPointInfo.EndpointAddress; }
		const TCHAR* GetName() const { return *DataPointInfo.Name; }
		bool IsSource() const { return DataPointInfo.bIsSource; }
		bool IsOnThisEndpoint() const { return DataPointInfo.bIsOnThisEndpoint; }
		bool IsPublic() const { return DataPointInfo.bIsPublic; }

#ifdef SWIG_FACADE
	private:
#endif
		explicit FDatasmithFacadeDataPointInfo(const DirectLink::FRawInfo::FDataPointInfo& InDataPointInfo)
			: DataPointInfo(InDataPointInfo)
		{}

	private:
		DirectLink::FRawInfo::FDataPointInfo DataPointInfo;
	};

	struct DATASMITHFACADE_API FDatasmithFacadeStreamInfo
	{
	public:
		uint32 GetStreamId() const { return StreamInfo.StreamId; }
		FGuid GetSource() const { return StreamInfo.Source; }
		FGuid GetDestination() const { return StreamInfo.Destination; }
		bool IsActive() const { return StreamInfo.ConnectionState == DirectLink::EStreamConnectionState::Active; }

#ifdef SWIG_FACADE
	private:
#endif
		explicit FDatasmithFacadeStreamInfo(const DirectLink::FRawInfo::FStreamInfo& InStreamInfo)
			: StreamInfo(InStreamInfo)
		{}

	private:
		DirectLink::FRawInfo::FStreamInfo StreamInfo;
	};

	/**
	 * Returns the MessageAddress associated to the current DirectLink Endpoint.
	 */
	FMessageAddress GetThisEndpointAddress() const { return RawInfo.ThisEndpointAddress; }

	/**
	 * Returns the Endpoint info associated to given MessageAddress, returns null if there is no Endpoint associated to the MessageAddress.
	 * The caller is responsible of deleting the returned object pointer.
	 */
	FDatasmithFacadeEndpointInfo* GetNewEndpointInfo(const FMessageAddress* MessageAddress) const;

	/**
	 * Returns the DataPointInfo (Source or Destination) associated to the given Id, returns null if there is no DataPointInfo associated to the Id.
	 * The caller is responsible of deleting the returned object pointer.
	 */
	FDatasmithFacadeDataPointInfo* GetNewDataPointsInfo(const FGuid* Id) const;

	int32 GetNumberOfStreamsInfo() const { return RawInfo.StreamsInfo.Num(); }

	/**
	 * Returns the Stream info at the given index, returns null if the index is invalid.
	 * The caller is responsible of deleting the returned object pointer.
	 */
	FDatasmithFacadeStreamInfo* GetNewStreamInfo(int32 Index) const;

#ifdef SWIG_FACADE
private:
#endif
	explicit FDatasmithFacadeRawInfo(const DirectLink::FRawInfo& InRawInfo)
		: RawInfo(InRawInfo)
	{}

private:
	DirectLink::FRawInfo RawInfo;
};

class FDatasmithFacadeEndpointObserverImpl;

class DATASMITHFACADE_API FDatasmithFacadeEndpointObserver
{
public:
	typedef void(*OnStateChangedDelegate)(FDatasmithFacadeRawInfo*);

	FDatasmithFacadeEndpointObserver();

	void RegisterOnStateChangedDelegateInternal(FDatasmithFacadeEndpointObserver::OnStateChangedDelegate InOnStateChanged);
	void UnregisterOnStateChangedDelegateInternal(FDatasmithFacadeEndpointObserver::OnStateChangedDelegate InOnStateChanged);

#ifdef SWIG_FACADE
private:
#endif

	const TSharedRef<FDatasmithFacadeEndpointObserverImpl>& GetObserver() const { return ObserverImpl; }

private:
	TSharedRef<FDatasmithFacadeEndpointObserverImpl> ObserverImpl;
};